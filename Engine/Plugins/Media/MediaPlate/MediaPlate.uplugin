{
	"FileVersion": 3,
	"Version": 0,
	"VersionName": "0.1",
	"FriendlyName": "Media Plate",
	"Description": "Actor that can play media.",
	"Category": "Media",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "https://docs.unrealengine.com/en-US/Engine/MediaFramework/Overview",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"EnabledByDefault": true,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": 
	[
		{
			"Name": "MediaPlate",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "MediaPlateEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": 
	[
		{
			"Name": "MediaCompositing",
			"TargetAllowList": [ "Editor" ],
			"Enabled": true
		},
		{
			"Name": "MediaPlayerEditor",
			"TargetAllowList": [ "Editor" ],
			"Enabled": true
		},
		{
			"Name": "MeshModelingToolset",
			"TargetAllowList": [ "Editor" ],
			"Enabled": true
		},
		{
			"Name": "CompositeCore",
			"Enabled": true
		}
	]
}