// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaPlate.h"

#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Engine/World.h"
#include "HoldoutCompositeComponent.h"
#include "CompositeCoreSubsystem.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "MediaPlateComponent.h"
#include "MediaPlateModule.h"
#include "MediaTexture.h"
#include "UObject/ConstructorHelpers.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#if WITH_EDITOR
#include "Editor.h"
#include "MediaPlateAssetUserData.h"
#include "Misc/MessageDialog.h"
#include "UObject/ObjectSaveContext.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaPlate)

#define LOCTEXT_NAMESPACE "MediaPlate"

FLazyName AMediaPlate::MediaPlateComponentName(TEXT("MediaPlateComponent0"));
FLazyName AMediaPlate::MediaTextureName("MediaTexture");

namespace UE::MediaPlate::Private
{
	/** Check for the presence of the (now deprecated) overlay composite material. */
	bool HasDeprecatedOverlayCompositeMaterial(AMediaPlate& InMediaPlate)
	{
		if (UMaterialInterface* OverlayMaterial = InMediaPlate.GetCurrentOverlayMaterial())
		{
			if (UMaterial* ParentMaterial = OverlayMaterial->GetMaterial())
			{
				return ParentMaterial->GetPathName() == TEXT("/MediaPlate/M_MediaPlate_OverlayComp.M_MediaPlate_OverlayComp");
			}
		}

		return false;
	}

	/** Returns the number of texture parameters that begin with the "MediaTexture" name string in the given material instance. */
	int32 GetNumMediaTextureParameters(const UMaterialInstance* InMaterialInstance, const FString& InMediaTextureString)
	{
		int32 NumTextures = 0;
		for (const FTextureParameterValue& Param : InMaterialInstance->TextureParameterValues)
		{
			const FString Name = Param.ParameterInfo.Name.ToString();
			if (Name.StartsWith(InMediaTextureString))
			{
				++NumTextures;
			}
		}
		return NumTextures;
	}

	/** Returns the expected material texture parameter name for the given texture index. */
	FName GetTextureParameterName(const FString& InMediaTextureString, int32 InTextureIndex)
	{
		// No index is appended for index zero.
		if (InTextureIndex == 0)
		{
			return FName(InMediaTextureString);
		}

		FString TextureParameterName = InMediaTextureString;
		TextureParameterName.AppendInt(InTextureIndex);
		return FName(TextureParameterName);
	}
}

AMediaPlate::AMediaPlate(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	RootComponent = CreateDefaultSubobject<USceneComponent>(TEXT("Root"));

	// Set up media component.
	MediaPlateComponent = CreateDefaultSubobject<UMediaPlateComponent>(MediaPlateComponentName);

	// Set up static mesh component.
	StaticMeshComponent = CreateDefaultSubobject<UStaticMeshComponent>(TEXT("StaticMeshComponent"));
	StaticMeshComponent->SetupAttachment(RootComponent);
	StaticMeshComponent->bCastStaticShadow = false;
	StaticMeshComponent->bCastDynamicShadow = false;
	MediaPlateComponent->StaticMeshComponent = StaticMeshComponent;

	// 16:9 by default since most videos are that format.
	MediaPlateComponent->SetAspectRatio(16.0f/9.0f);

	// Hook up mesh.
	struct FConstructorStatics
	{
		ConstructorHelpers::FObjectFinder<UStaticMesh> Plane;
		FConstructorStatics()
			: Plane(TEXT("/MediaPlate/SM_MediaPlateScreen"))
		{}
	};

	static FConstructorStatics ConstructorStatics;
	if (ConstructorStatics.Plane.Object != nullptr)
	{
		StaticMeshComponent->SetStaticMesh(ConstructorStatics.Plane.Object);
	}

#if WITH_EDITOR
	// If we are not the class default object...
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		// Hook into pre/post save.
		FEditorDelegates::PreSaveWorldWithContext.AddUObject(this, &AMediaPlate::OnPreSaveWorld);
		FEditorDelegates::PostSaveWorldWithContext.AddUObject(this, &AMediaPlate::OnPostSaveWorld);
	}
#endif // WITH_EDITOR
}

void AMediaPlate::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	// The holdout composite component was replaced with a checkbox.
	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::MediaPlateHoldoutComponentRemoval)
	{
		if (UHoldoutCompositeComponent* HoldoutCompositeComponent = FindComponentByClass<UHoldoutCompositeComponent>())
		{
			bEnableHoldoutComposite = HoldoutCompositeComponent->IsEnabled();

			HoldoutCompositeComponent->DestroyComponent();
		}
	}

	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::MediaPlateOverlayTechniqueRemoval)
	{
		if (UE::MediaPlate::Private::HasDeprecatedOverlayCompositeMaterial(*this))
		{
			UE_LOGF(LogMediaPlate, Display, "Deprecated overlay material technique automatically replaced with holdout composite.");

			StaticMeshComponent->SetOverlayMaterial(nullptr);

			bEnableHoldoutComposite = true;
		}
	}
#endif
}

void AMediaPlate::PostActorCreated()
{
	Super::PostActorCreated();

#if WITH_EDITOR
	if (GIsEditor)
	{
		// Set which material to use.
		UseDefaultMaterial();
	}
#endif
}

void AMediaPlate::PostRegisterAllComponents()
{
	Super::PostRegisterAllComponents();

#if WITH_EDITOR
	// If this media plate is in a Sequencer media track,
	// and we go though a convert to spawnable/possessible,
	// then a multi user client will not receive the correct material
	// as multi user cannot send materials that are not assets.
	// So if we have an override material but its nullptr, then just use the default material.
	if ((StaticMeshComponent != nullptr) &&
		(StaticMeshComponent->GetNumOverrideMaterials() > 0) &&
		(StaticMeshComponent->OverrideMaterials[0] == nullptr))
	{
		UseDefaultMaterial();		
	}
	
	AddAssetUserData();
#endif // WITH_EDITOR

	if (IsValid(StaticMeshComponent))
	{
		ConditionallyEnableHoldoutComposite();

#if WITH_EDITOR
		// Update cached "last" members to prevent unnecessary material updates on static mesh edits (see ApplyCurrentMaterial).
		LastMaterial = StaticMeshComponent->GetMaterial(0);
		LastOverlayMaterial = StaticMeshComponent->GetOverlayMaterial();
#endif
	}
}

void AMediaPlate::BeginDestroy()
{
	Super::BeginDestroy();
}

void AMediaPlate::SetHoldoutCompositeEnabled(bool bInEnabled)
{
	bEnableHoldoutComposite = bInEnabled;

	UCompositeCoreSubsystem* CompositeSubsystem = UWorld::GetSubsystem<UCompositeCoreSubsystem>(GetWorld());
	if (IsValid(CompositeSubsystem))
	{
		if (bEnableHoldoutComposite)
		{
			// Note: re-registering the same component is safe.
			CompositeSubsystem->RegisterPrimitive(StaticMeshComponent);
		}
		else
		{
			// Note: destroyed components are also automatically removed by the system.
			CompositeSubsystem->UnregisterPrimitive(StaticMeshComponent);
		}
	}
}

bool AMediaPlate::IsHoldoutCompositeEnabled() const
{
	return bEnableHoldoutComposite;
}

UMaterialInterface* AMediaPlate::GetCurrentMaterial() const
{
	if (IsValid(StaticMeshComponent))
	{
		return StaticMeshComponent->GetMaterial(0);
	}

	return nullptr;
}

UMaterialInterface* AMediaPlate::GetCurrentOverlayMaterial() const
{
	if (IsValid(StaticMeshComponent))
	{
		return StaticMeshComponent->GetOverlayMaterial();
	}

	return nullptr;
}

void AMediaPlate::ConditionallyEnableHoldoutComposite()
{
	if (bEnableHoldoutComposite)
	{
		SetHoldoutCompositeEnabled(true);
	}
}

#if WITH_EDITOR

void AMediaPlate::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(ThisClass, bEnableHoldoutComposite))
	{
		SetHoldoutCompositeEnabled(bEnableHoldoutComposite);
	}
}

void AMediaPlate::OnStaticMeshChange()
{
	ApplyCurrentMaterial();

	ConditionallyEnableHoldoutComposite();
}

void AMediaPlate::UseDefaultMaterial()
{
	UMaterial* DefaultMaterial = LoadObject<UMaterial>(NULL, TEXT("/MediaPlate/M_MediaPlate_Opaque"), NULL, LOAD_None, NULL);
	
	ApplyMaterial(DefaultMaterial);

	if (IsValid(StaticMeshComponent))
	{
		StaticMeshComponent->SetOverlayMaterial(nullptr);
		
		LastOverlayMaterial = nullptr;
	}
}

void AMediaPlate::ApplyCurrentMaterial()
{
	UMaterialInterface* MaterialInterface = GetCurrentMaterial();
	
	if ((MaterialInterface != nullptr) && (LastMaterial != MaterialInterface))
	{
		ApplyMaterial(MaterialInterface);
	}

	UMaterialInterface* OverlayMaterialInterface = GetCurrentOverlayMaterial();

	if ((OverlayMaterialInterface != nullptr) && (LastOverlayMaterial != OverlayMaterialInterface))
	{
		ApplyOverlayMaterial(OverlayMaterialInterface);
	}
}

UMaterialInterface* AMediaPlate::CreateMaterialInstanceConstant(UMaterialInterface* InMaterial)
{
	// Change M_ to MI_ in material name and then generate a unique one.
	FString MaterialName = InMaterial->GetName();
	if (MaterialName.StartsWith(TEXT("M_")))
	{
		MaterialName.InsertAt(1, TEXT("I"));
	}
	FName MaterialUniqueName = MakeUniqueObjectName(StaticMeshComponent, UMaterialInstanceConstant::StaticClass(),
		FName(*MaterialName));

	// Create instance.
	UMaterialInstanceConstant* MaterialInstance =
		NewObject<UMaterialInstanceConstant>(StaticMeshComponent, MaterialUniqueName, RF_Transactional);
	MaterialInstance->SetParentEditorOnly(InMaterial);
	MaterialInstance->CopyMaterialUniformParametersEditorOnly(InMaterial);

	using namespace UE::MediaPlate::Private;
	const FString MediaTextureNameString = MediaTextureName.Resolve().ToString();
	const int32 NumTextures = GetNumMediaTextureParameters(MaterialInstance, MediaTextureNameString);

	if (NumTextures == 0)
	{
		UE_LOGF(LogMediaPlate, Warning, "Material \"%ls\" doesn't have any texture parameters that start with \"%ls\".", *MaterialName, *MediaTextureNameString);
	}
	
	MediaPlateComponent->SetNumberOfTextures(FMath::Max(NumTextures, 1));

	MaterialInstance->SetTextureParameterValueEditorOnly(
		FMaterialParameterInfo(MediaTextureName),
		MediaPlateComponent->GetMediaTexture());

	for (int32 TextureIndex = 1; TextureIndex < NumTextures; ++TextureIndex)
	{
		MaterialInstance->SetTextureParameterValueEditorOnly(
			FMaterialParameterInfo(GetTextureParameterName(MediaTextureNameString, TextureIndex)),
			MediaPlateComponent->GetMediaTexture(TextureIndex));
	}

	// Cross fade material: make sure the blend0 weight is set.
	MaterialInstance->SetScalarParameterValueEditorOnly(FMaterialParameterInfo(TEXT("Blend0")), 1.0f);

	MaterialInstance->PostEditChange();

	// We force call post-load to indirectly call UpdateParameters() (for integration with VPUtilities plugin).
	MaterialInstance->PostLoad();

	return MaterialInstance;
}

void AMediaPlate::ApplyMaterial(UMaterialInterface* Material)
{
	if (Material != nullptr && StaticMeshComponent != nullptr)
	{
		if (GEditor == nullptr)
		{
			UMaterialInstanceDynamic* MaterialDynamic = StaticMeshComponent->CreateAndSetMaterialInstanceDynamicFromMaterial(0, Material);
			SetMIDParameters(MaterialDynamic);
			LastMaterial = MaterialDynamic;
		}
		else
		{
			UMaterialInterface* Result = nullptr;

			// See if we can modify this material.
			bool bCanModify = true;
			FMediaPlateModule* MediaPlateModule = FModuleManager::GetModulePtr<FMediaPlateModule>("MediaPlate");
			if (MediaPlateModule != nullptr)
			{
				MediaPlateModule->OnMediaPlateApplyMaterial.Broadcast(Material, this, bCanModify);
			}

			if (bCanModify == false)
			{
				MediaPlateComponent->SetNumberOfTextures(1);
				LastMaterial = Material;
			}
			else if (UMaterialInstanceDynamic* InMID = Cast<UMaterialInstanceDynamic>(Material))
			{
				SetMIDParameters(InMID);
				Result = InMID;
			}
			else if (Material->IsA(UMaterialInstance::StaticClass()))
			{
				UMaterialInstanceDynamic* MID = UMaterialInstanceDynamic::Create(Material, StaticMeshComponent);
				SetMIDParameters(MID);
				Result = MID;
			}
			else
			{
				Result = CreateMaterialInstanceConstant(Material);
			}

			// Update static mesh.
			if (Result != nullptr)
			{
				StaticMeshComponent->Modify();
				StaticMeshComponent->SetMaterial(0, Result);

				LastMaterial = Result;
			}
		}
	}
}

void AMediaPlate::ApplyOverlayMaterial(UMaterialInterface* InOverlayMaterial)
{
	if (InOverlayMaterial != nullptr && StaticMeshComponent != nullptr)
	{
		UMaterialInstanceDynamic* MID = Cast<UMaterialInstanceDynamic>(InOverlayMaterial);

		if (GEditor == nullptr)
		{
			if (MID == nullptr)
			{
				// Create and set the dynamic material instance.
				MID = UMaterialInstanceDynamic::Create(InOverlayMaterial, StaticMeshComponent);
			}
			StaticMeshComponent->SetOverlayMaterial(MID);
			SetMIDParameters(MID);
			LastOverlayMaterial = MID;
		}
		else
		{
			UMaterialInterface* Result = nullptr;

			if (MID != nullptr)
			{
				SetMIDParameters(MID);
				Result = MID;
			}
			else if (InOverlayMaterial->IsA(UMaterialInstance::StaticClass()))
			{
				UMaterialInstanceDynamic* OverlayMID = UMaterialInstanceDynamic::Create(InOverlayMaterial, StaticMeshComponent);
				SetMIDParameters(OverlayMID);
				Result = OverlayMID;
			}
			else
			{
				Result = CreateMaterialInstanceConstant(InOverlayMaterial);
			}

			// Update static mesh.
			if (Result != nullptr)
			{
				StaticMeshComponent->Modify();
				StaticMeshComponent->SetOverlayMaterial(Result);

				LastOverlayMaterial = Result;
			}
		}
	}
}

void AMediaPlate::SetMIDParameters(UMaterialInstanceDynamic* InMaterial)
{
	using namespace UE::MediaPlate::Private;
	const FString MediaTextureString = MediaTextureName.Resolve().ToString();
	const int32 NumTextures = GetNumMediaTextureParameters(InMaterial, MediaTextureString);

	if (NumTextures == 0)
	{
		UE_LOGF(LogMediaPlate, Warning, "Material \"%ls\" doesn't have any texture parameters that start with \"%ls\".", *InMaterial->GetName(), *MediaTextureString);
	}
	
	MediaPlateComponent->SetNumberOfTextures(FMath::Max(NumTextures, 1));

	InMaterial->SetTextureParameterValue(MediaTextureName, MediaPlateComponent->GetMediaTexture());
	
	for (int32 Index = 1; Index < NumTextures; Index++)
	{
		InMaterial->SetTextureParameterValue(GetTextureParameterName(MediaTextureString, Index),
			MediaPlateComponent->GetMediaTexture(Index));
	}
}

void AMediaPlate::OnPreSaveWorld(UWorld* InWorld, FObjectPreSaveContext ObjectSaveContext)
{
	// We need to remove our asset user data before saving, as we do not need to save it out
	// and only use it to know when the static mesh component changes.
	RemoveAssetUserData();
}

void AMediaPlate::OnPostSaveWorld(UWorld* InWorld, FObjectPostSaveContext ObjectSaveContext)
{
	AddAssetUserData();
}

void AMediaPlate::AddAssetUserData()
{
	if (StaticMeshComponent != nullptr && !StaticMeshComponent->HasAssetUserDataOfClass(UMediaPlateAssetUserData::StaticClass()))
	{
		UMediaPlateAssetUserData* AssetUserData = NewObject<UMediaPlateAssetUserData>(GetTransientPackage());
		AssetUserData->OnPostEditChangeOwner.BindUObject(this, &AMediaPlate::OnStaticMeshChange);
		StaticMeshComponent->AddAssetUserData(AssetUserData);
	}
}

void AMediaPlate::RemoveAssetUserData()
{
	if (StaticMeshComponent != nullptr)
	{
		StaticMeshComponent->RemoveUserDataOfClass(UMediaPlateAssetUserData::StaticClass());
	}
}

#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE

