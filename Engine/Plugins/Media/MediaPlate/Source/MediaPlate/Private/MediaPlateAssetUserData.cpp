// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaPlateAssetUserData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaPlateAssetUserData)

void UMediaPlateAssetUserData::PostEditChangeOwner(const FPropertyChangedEvent& PropertyChangedEvent)
{
	OnPostEditChangeOwner.ExecuteIfBound();
}
