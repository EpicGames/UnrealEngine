// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaPlateComponent.h"

#include "Components/StaticMeshComponent.h"
#include "Engine/Engine.h"
#include "Engine/StaticMesh.h"
#include "HAL/FileManager.h"
#include "IMediaAssetsModule.h"
#include "IMediaClock.h"
#include "IMediaClockSink.h"
#include "IMediaModule.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "MediaPlate.h"
#include "MediaPlateModule.h"
#include "MediaPlateOpenLatentAction.h"
#include "MediaPlayer.h"
#include "MediaPlaylist.h"
#include "MediaSoundComponent.h"
#include "MediaSource.h"
#include "MediaTexture.h"
#include "MediaTextureTracker.h"

#if WITH_EDITOR
#include "ScopedTransaction.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaPlateComponent)

#define LOCTEXT_NAMESPACE "MediaPlate"

namespace UE::MediaPlateComponent
{
	enum class ESetUpTexturesFlags
	{
		None,
		AllowSetPlayer = 0x1,
		ForceUpdateResource = 0x2,
	};
	ENUM_CLASS_FLAGS(ESetUpTexturesFlags);

	// Runs through media textures and sets Media Plate settings corresponding to Media Texture.
	void ApplyMediaTextureMipGenProperties(const FMediaTextureResourceSettings MediaTextureSettings, const TArray<TObjectPtr<UMediaTexture>>& MediaTextures)
	{
		for (TObjectPtr<UMediaTexture> MediaTexture : MediaTextures)
		{
			if (MediaTexture != nullptr
				&& (MediaTexture->EnableGenMips != MediaTextureSettings.bEnableGenMips))
			{
				MediaTexture->EnableGenMips = MediaTextureSettings.bEnableGenMips;
				MediaTexture->UpdateResource();
			}
		}
	}

	void EnsureMediaTexturePropertiesInSync(const FMediaTextureResourceSettings MediaTextureSettings, const TArray<TObjectPtr<UMediaTexture>>& MediaTextures)
	{
#if !UE_BUILD_SHIPPING
		for (TObjectPtr<UMediaTexture> MediaTexture : MediaTextures)
		{
			if (MediaTexture != nullptr)
			{
				const bool bMediaTextureMipGenPropertiesInSync = MediaTexture->EnableGenMips == MediaTextureSettings.bEnableGenMips;
				ensureMsgf(bMediaTextureMipGenPropertiesInSync, TEXT("Mip Generation properties set on Media Plate are different from the properties set on Media Texture. \n\
					Media Texture mip generation properites are not meant to be modified directly."));
			}
		}
#endif
	}

	// Find the media texture associated with the given media player and update it's resource to reset it.
	void ResetMediaTextureForPlayer(const UMediaPlayer* InMediaPlayer, const TArray<TObjectPtr<UMediaTexture>>& InMediaTextures)
	{
		for (TObjectPtr<UMediaTexture> MediaTexture : InMediaTextures)
		{
			if (MediaTexture && MediaTexture->GetMediaPlayer() == InMediaPlayer)
			{
				MediaTexture->UpdateResource();
			}
		}
	}

	UMediaSource* GetMediaSource(UMediaPlaylist* InPlaylist, int32 InPlaylistIndex)
	{
		return InPlaylist ? InPlaylist->Get(InPlaylistIndex) : nullptr;
	}

#if WITH_EDITOR
	/**
	 * Legacy support code to extract a media path compatible with SpawnMediaSourceForString from the
	 * existing MediaSource.
	 * 
	 * @param InExternalMediaSource Existing media source we want to extract a path from.
	 * @param InOuter Outer object for the temporary creation of a media source to validate.
	 * @return Extracted path.
	 */
	FString ExtractExternalMediaPath(const UMediaSource* InExternalMediaSource, UObject* InOuter)
	{
		FString MediaPath = InExternalMediaSource->GetUrl();

		const FString FilePrefix(TEXT("file://"));
		const FString ImgPrefix(TEXT("img://"));
		if (MediaPath.StartsWith(FilePrefix))
		{
			MediaPath = MediaPath.RightChop(FilePrefix.Len());
		}
		else if (MediaPath.StartsWith(ImgPrefix))
		{
			// Special case of img sequence.
			// We need to extract a path that is compatible with SpawnMediaSourceForString.
			// As it currently stands, this requires a valid file path with extension.
			
			MediaPath = MediaPath.RightChop(ImgPrefix.Len());

			// This is going to be a directory name
			if (FPaths::DirectoryExists(MediaPath))
			{
				// Get the first compatible file.
				TArray<FString> FoundFiles;
				IFileManager::Get().FindFiles(FoundFiles, *MediaPath, TEXT("*"));
				for (const FString& File : FoundFiles)
				{
					const FString Fullpath = FPaths::Combine(MediaPath, File);
					if (UMediaSource::SpawnMediaSourceForString(Fullpath, InOuter))
					{
						MediaPath = Fullpath;
						break;
					}
				}
			}
		}

		// Try to make relative under content or project, if not leave absolute.
		FString ConvertedPath = FPaths::ConvertRelativePathToFull(MediaPath);

		const TArray<FString> PossibleBasePaths =
		{
			FPaths::ProjectContentDir(),
			FPaths::ProjectDir()
		};

		for (const FString& BasePath : PossibleBasePaths)
		{
			const FString FullBasePath = FPaths::ConvertRelativePathToFull(BasePath);
			if (FPaths::IsUnderDirectory(ConvertedPath, FullBasePath) && FPaths::MakePathRelativeTo(ConvertedPath, *FullBasePath))
			{
				// Relative path expected to start with "./"
				if (!ConvertedPath.StartsWith(TEXT("./")))
				{
					ConvertedPath = FPaths::Combine(TEXT("."), ConvertedPath);
				}
				break;
			}
		}
		return ConvertedPath;
	}
#endif
};

/**
 * Media clock sink for media textures.
 */
class FMediaComponentClockSink
	: public IMediaClockSink
{
public:

	FMediaComponentClockSink(UMediaPlateComponent* InOwner)
		: Owner(InOwner)
	{ }

	virtual ~FMediaComponentClockSink() { }

	virtual void TickOutput(FTimespan DeltaTime, FTimespan Timecode) override
	{
		if (UMediaPlateComponent* OwnerPtr = Owner.Get())
		{
			Owner->TickOutput();
		}
	}


	/**
	 * Call this when the owner is destroyed.
	 */
	void OwnerDestroyed()
	{
		Owner.Reset();
	}

private:

	TWeakObjectPtr<UMediaPlateComponent> Owner;
};

UMediaPlateComponent::UMediaPlateComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bTickInEditor = true;
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.bStartWithTickEnabled = false;

	CacheSettings.bOverride = true;

#if WITH_EDITORONLY_DATA
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// Still create the deprecated MediaPlaylist so the old data can be serialized during loading
	MediaPlaylist_DEPRECATED = CreateDefaultSubobject<UMediaPlaylist>("MediaPlaylist0");
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

	// Default to plane since AMediaPlate defaults to SM_MediaPlateScreen
	VisibleMipsTilesCalculations = EMediaTextureVisibleMipsTiles::Plane;

	MediaTextureSettings.bEnableGenMips = false;
}

#if WITH_EDITOR
void UMediaPlateComponent::PostLoad()
{
	Super::PostLoad();

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// Use the old media texture if we have one.
	if (MediaTexture_DEPRECATED != nullptr)
	{
		if (MediaTextures.Num() == 0)
		{
			MediaTextures.Add(MediaTexture_DEPRECATED);
		}
		MediaTexture_DEPRECATED = nullptr;
	}

	UE::MediaPlateComponent::ApplyMediaTextureMipGenProperties(MediaTextureSettings, MediaTextures);

	if (MediaPlaylist_DEPRECATED)
	{
		InitializeMediaPlateResource();
	}

	if(MediaPlateResource.ExternalMedia_DEPRECATED)
	{
		// Move the existing media source.
		ExternalMediaSource = MediaPlateResource.ExternalMedia_DEPRECATED;
		MediaPlateResource.ExternalMedia_DEPRECATED = nullptr;
		
		// Update external media path to match (as best we can).
		MediaPlateResource.LoadExternalMedia(UE::MediaPlateComponent::ExtractExternalMediaPath(ExternalMediaSource, this));
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	// Ensure the external media source is loaded.
	if (MediaPlateResource.Type == EMediaPlateResourceType::External && !ExternalMediaSource)
	{
		RefreshExternalMediaSource();
	}

	// Ensure active playlist is refreshed.
	if (!ActivePlaylist)
	{
		RefreshActivePlaylist();
	}
}
#endif // WITH_EDITOR

void UMediaPlateComponent::OnRegister()
{
	Super::OnRegister();

	// Create media texture if we don't have one.
	if (MediaTextures.Num() == 0)
	{
		SetNumberOfTextures(1);
	}

	// Create media player if we don't have one.
	if (MediaPlayer == nullptr)
	{
		MediaPlayer = NewObject<UMediaPlayer>(this);
		MediaPlayer->SetLooping(false);
		MediaPlayer->PlayOnOpen = false;
	}
	MediaPlayer->OnMediaOpened.AddUniqueDynamic(this, &UMediaPlateComponent::OnMediaOpened);
	MediaPlayer->OnEndReached.AddUniqueDynamic(this, &UMediaPlateComponent::OnMediaEnd);
	MediaPlayer->OnPlaybackResumed.AddUniqueDynamic(this, &UMediaPlateComponent::OnMediaResumed);
	MediaPlayer->OnPlaybackSuspended.AddUniqueDynamic(this, &UMediaPlateComponent::OnMediaSuspended);

	// Set up media texture. Note: will not set player to proxied textures.
	SetUpTextures(UE::MediaPlateComponent::ESetUpTexturesFlags::AllowSetPlayer);

	// Set up sound component if we have one.
	if (SoundComponent != nullptr)
	{
		if (MediaPlayer != nullptr)
		{
			SoundComponent->SetMediaPlayer(MediaPlayer);
		}
	}

	RegisterWithMediaTextureTracker();
}

void UMediaPlateComponent::BeginPlay()
{
	Super::BeginPlay();

	// Start playing?
	if (bAutoPlay)
	{
		Open();
	}
}

void UMediaPlateComponent::EndPlay(const EEndPlayReason::Type InEndPlayReason)
{
	// Ensure media resources are properly released.
	Close();

	Super::EndPlay(InEndPlayReason);
}

void UMediaPlateComponent::BeginDestroy()
{
	if (ClockSink.IsValid())
	{
		// Tell sink we are done.
		ClockSink->OwnerDestroyed();

		IMediaModule* MediaModule = FModuleManager::GetModulePtr<IMediaModule>("Media");
		if (MediaModule != nullptr)
		{
			MediaModule->GetClock().RemoveSink(ClockSink.ToSharedRef());
		}

		ClockSink.Reset();
	}

	// Ensure media resources are properly released.
	Close();
	
	Super::BeginDestroy();
}


void UMediaPlateComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	// Making sure that Media Plate enforces the same settings to Media textures in case these settings were modified externally.
	UE::MediaPlateComponent::EnsureMediaTexturePropertiesInSync(MediaTextureSettings, MediaTextures);

	if (MediaPlayer != nullptr)
	{
		// Pending rate changes?
		if (IntendedPlaybackState != ActualPlaybackState)
		{
			if (IntendedPlaybackState != PendingPlaybackState)
			{
				if (IntendedPlaybackState == EPlaybackState::Resume)
				{
					FTimespan PlayTime = GetResumeTime();
					MediaPlayer->Seek(PlayTime);
					IntendedPlaybackState = EPlaybackState::Playing;
				}
				PendingPlaybackState = IntendedPlaybackState;
				if (IntendedPlaybackState == EPlaybackState::Playing)
				{
					if (FMath::IsNearlyZero(CurrentRate) || !MediaPlayer->SupportsRate(CurrentRate, /* Unthinned */ true))
					{
						CurrentRate = 1.f;
					}

					MediaPlayer->SetRate(CurrentRate);
				}
				else
				{
					MediaPlayer->Pause();
				}
			}
		}
		else
		{
			PendingPlaybackState = EPlaybackState::Unset;
		}

		// Perform visibility check only when not currently transitioning.
		if (PendingPlaybackState == EPlaybackState::Unset)
		{
			if ((CurrentRate != 0.0f) || bWantsToPlayWhenVisible)
			{
				bool bIsVisible = IsVisible();
				if (bIsVisible)
				{
					ResumeWhenVisible();
				}
				else if (ActualPlaybackState == EPlaybackState::Playing)
				{
					IntendedPlaybackState = EPlaybackState::Paused;
					TimeWhenPlaybackPaused = FApp::GetGameTime();
				}
			}
		}
	}
}

void UMediaPlateComponent::OnUnregister()
{
	Super::OnUnregister();

	UnregisterWithMediaTextureTracker();
}

void UMediaPlateComponent::OnComponentDestroyed(bool bDestroyingHierarchy)
{
	// Free the resources as soon as the component is destroyed (sooner than GC).
	// If waiting for GC, it looks like resources are leaking.
	Close();

	Super::OnComponentDestroyed(bDestroyingHierarchy);
}

UMediaPlayer* UMediaPlateComponent::GetMediaPlayer()
{
	return MediaPlayer;
}

UMediaTexture* UMediaPlateComponent::GetMediaTexture(int32 Index)
{
	UMediaTexture* MediaTexture = nullptr;
	if ((Index >= 0) && (Index < MediaTextures.Num()))
	{
		MediaTexture = MediaTextures[Index];
	}
	else
	{
		UE_CALL_ONCE([Index](){
			UE_LOGF(LogMediaPlate, Warning, "Material does not support texture index %d. Either remove the number of cross fades or change the material.", Index);
		});
	}
	return MediaTexture;
}

bool UMediaPlateComponent::IsEventStateChangeAllowed(EMediaPlateEventState InEventState) const
{
	// Note: centralizing the state change restriction here to make it easier to maintain.
	switch(InEventState)
	{
	case EMediaPlateEventState::Open:
	case EMediaPlateEventState::Play:
		return !IsAnyMediaTextureProxied();
	default:
		return true;
	}
}

void UMediaPlateComponent::Open()
{
	bIsMediaPlatePlaying = true;
	CurrentRate = bPlayOnOpen ? 1.0f : 0.0f;
	IntendedPlaybackState = bPlayOnOpen ? EPlaybackState::Playing : EPlaybackState::Paused;
	PendingPlaybackState = EPlaybackState::Unset;
	ActualPlaybackState = EPlaybackState::Paused;
	TimeWhenPlaybackPaused = -1.0;

	PlaylistIndex = 0;
	SetNormalMode(true);

	if (IsVisible())
	{
		if (MediaPlayer != nullptr)
		{
			if (UMediaSource* MediaSource = UE::MediaPlateComponent::GetMediaSource(GetMediaPlaylist(), PlaylistIndex))
			{
				if (!PlayMediaSource(MediaSource, bPlayOnOpen))
				{
					UE_LOGF(LogMediaPlate, Warning, "Open Failed: failed to open media source \"%ls\".", *MediaSource->GetUrl());	
				}
			}
			else
			{
				UE_LOGF(LogMediaPlate, Warning, "Open Failed: no media source specified.");
			}
		}
		else
		{
			UE_LOGF(LogMediaPlate, Warning, "Open Failed: missing media player.");
		}
	}
	else
	{
		bWantsToPlayWhenVisible = true;
		TimeWhenPlaybackPaused = FApp::GetGameTime();
	}

	UpdateTicking();
}

void UMediaPlateComponent::OpenLatent(const UObject* InWorldContextObject, struct FLatentActionInfo InLatentInfo, float InTimeout, bool bInWaitForTexture, bool& bOutSuccess)
{
	bOutSuccess = false;

	if (UWorld* World = GEngine->GetWorldFromContextObject(InWorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		FLatentActionManager& LatentManager = World->GetLatentActionManager();
		if (LatentManager.FindExistingAction<FMediaPlateOpenLatentAction>(InLatentInfo.CallbackTarget, InLatentInfo.UUID) == nullptr)
		{
			FMediaPlateOpenLatentAction* NewAction = new FMediaPlateOpenLatentAction(InLatentInfo, this, InTimeout, bInWaitForTexture, bOutSuccess);
			LatentManager.AddNewAction(InLatentInfo.CallbackTarget, InLatentInfo.UUID, NewAction);
		}
	}
}

bool UMediaPlateComponent::Next()
{
	UMediaPlaylist* Playlist = GetMediaPlaylist();
	if (!Playlist)
	{
		return false;
	}

	bool bIsSuccessful = false;

	// Do we have a playlist?
	if (Playlist->Num() > 1)
	{
		if ((PlaylistIndex < Playlist->Num() - 1) || (bLoop))
		{
			// Get the next media to play.
			UMediaSource* NextSource = Playlist->GetNext(PlaylistIndex);
			if (NextSource != nullptr)
			{
				bIsSuccessful = PlayMediaSource(NextSource, true);
			}
		}
	}

	return bIsSuccessful;
}

void UMediaPlateComponent::Play()
{
	IntendedPlaybackState = EPlaybackState::Playing;
	CurrentRate = 1.0f;
}

void UMediaPlateComponent::PlayReverse()
{
	IntendedPlaybackState = EPlaybackState::Playing;
	CurrentRate = -1.f;
}

void UMediaPlateComponent::Pause()
{
	IntendedPlaybackState = EPlaybackState::Paused;
	CurrentRate = 0.0f;
}

bool UMediaPlateComponent::Previous()
{
	UMediaPlaylist* Playlist = GetMediaPlaylist();
	if (!Playlist)
	{
		return false;
	}

	bool bIsSuccessful = false;

	// Do we have a playlist?
	if (Playlist->Num() > 1)
	{
		// Get the previous media to play.
		if (PlaylistIndex > 0)
		{
			UMediaSource* NextSource = Playlist->GetPrevious(PlaylistIndex);
			if (NextSource != nullptr)
			{
				bIsSuccessful = PlayMediaSource(NextSource, true);
			}
		}
	}

	return bIsSuccessful;
}

bool UMediaPlateComponent::Rewind()
{
	return Seek(FTimespan::Zero());
}

bool UMediaPlateComponent::Seek(const FTimespan& Time)
{
	if (MediaPlayer != nullptr)
	{
		return MediaPlayer->Seek(Time);
	}

	return false;
}

void UMediaPlateComponent::Close()
{
	if (MediaPlayer != nullptr)
	{
		MediaPlayer->Close();
	}

	StopClockSink();
	bIsMediaPlatePlaying = false;
	bWantsToPlayWhenVisible = false;
	bResumeWhenOpened = false;
	PlaylistIndex = 0;
	UpdateTicking();
}

bool UMediaPlateComponent::GetLoop()
{
	return bLoop;
}

void UMediaPlateComponent::SetLoop(bool bInLoop)
{
	if (bLoop != bInLoop)
	{
		bLoop = bInLoop;
		if (MediaPlayer != nullptr)
		{
			MediaPlayer->SetLooping(bLoop);
		}
	}
}

UMediaPlaylist* UMediaPlateComponent::GetMediaPlaylist() const
{
	return ActivePlaylist;
}

UMediaSource* UMediaPlateComponent::GetSelectedMediaSource() const
{
	switch (MediaPlateResource.GetResourceType())
	{
	case EMediaPlateResourceType::Playlist:
		return nullptr;

	case EMediaPlateResourceType::External:
		return ExternalMediaSource;

	case EMediaPlateResourceType::Asset:
		return MediaPlateResource.GetMediaAsset();

	default:
		return nullptr;
	}
}

void UMediaPlateComponent::SelectExternalMedia(const FString& InFilePath)
{
	MediaPlateResource.LoadExternalMedia(InFilePath);
	RefreshMediaPlateResource();
}

void UMediaPlateComponent::SelectMediaSourceAsset(const UMediaSource* InMediaSource)
{
	MediaPlateResource.SelectAsset(InMediaSource);
	RefreshMediaPlateResource();
}

void UMediaPlateComponent::SelectMediaPlaylistAsset(const UMediaPlaylist* InMediaPlaylist)
{
	MediaPlateResource.SelectPlaylist(InMediaPlaylist);
	RefreshMediaPlateResource();
}

void UMediaPlateComponent::SetMediaPlateResource(const FMediaPlateResource& InMediaPlayerResource)
{
#if WITH_EDITOR
	const FScopedTransaction Transaction(LOCTEXT("SetMediaPlateResource", "Set media plate resource"));
	this->Modify();
	if (GetMediaPlaylist())
	{
		GetMediaPlaylist()->Modify();
	}
#endif

	MediaPlateResource.Init(InMediaPlayerResource);
	RefreshMediaPlateResource();
}

void UMediaPlateComponent::SetEnableAudio(bool bInEnableAudio)
{
	if (bEnableAudio != bInEnableAudio)
	{
		bEnableAudio = bInEnableAudio;
		RefreshMediaSoundComponent();
	}
}

void UMediaPlateComponent::SetMeshRange(FVector2D InMeshRange)
{
	MeshRange = InMeshRange;

	if (MediaTextureTrackerObject != nullptr)
	{
		MediaTextureTrackerObject->MeshRange = MeshRange;
	}
}

void UMediaPlateComponent::SetPlayOnlyWhenVisible(bool bInPlayOnlyWhenVisible)
{
	bPlayOnlyWhenVisible = bInPlayOnlyWhenVisible;
	PlayOnlyWhenVisibleChanged();
}

void UMediaPlateComponent::SetIsAspectRatioAuto(bool bInIsAspectRatioAuto)
{
	if (bIsAspectRatioAuto != bInIsAspectRatioAuto)
	{
		bIsAspectRatioAuto = bInIsAspectRatioAuto;
		TryActivateAspectRatioAuto();
	}
}

void UMediaPlateComponent::PlayOnlyWhenVisibleChanged()
{
	// If we are turning off PlayOnlyWhenVisible then make sure we are playing.
	if (bPlayOnlyWhenVisible == false)
	{
		ResumeWhenVisible();
	}
}

void UMediaPlateComponent::RegisterWithMediaTextureTracker()
{
	UnregisterWithMediaTextureTracker();

	// Set up object.
	if (MediaTextureTrackerObject == nullptr)
	{
		MediaTextureTrackerObject = MakeShared<FMediaTextureTrackerObject, ESPMode::ThreadSafe>();
	}

	MediaTextureTrackerObject->Object = GetOwner();
	MediaTextureTrackerObject->MipMapLODBias = MipMapBias;
	MediaTextureTrackerObject->VisibleMipsTilesCalculations = VisibleMipsTilesCalculations;
	MediaTextureTrackerObject->MeshRange = MeshRange;
	MediaTextureTrackerObject->MipLevelToUpscale = bEnableMipMapUpscaling ? MipLevelToUpscale : -1;
	MediaTextureTrackerObject->bAdaptivePoleMipUpscaling = bAdaptivePoleMipUpscaling;

	// Specify view target resolution for any subsequent mip-level estimation, defaulting to render resolution.
	EMediaTextureTargetViewResolution TargetViewResolutionMask = EMediaTextureTargetViewResolution::RenderResolution;
	if (AMediaPlate* MediaPlate = GetOwner<AMediaPlate>())
	{
		// If holdout composite is enabled, we only use the display resolution.
		if (MediaPlate->IsHoldoutCompositeEnabled())
		{
			TargetViewResolutionMask = EMediaTextureTargetViewResolution::DisplayResolution;
		}
	}
	MediaTextureTrackerObject->TargetViewResolutionMask = TargetViewResolutionMask;

	// Add our textures.
	FMediaTextureTracker& MediaTextureTracker = FMediaTextureTracker::Get();
	for (UMediaTexture* MediaTexture : MediaTextures)
	{
		if (MediaTexture != nullptr)
		{
			MediaTextureTracker.RegisterTexture(MediaTextureTrackerObject, MediaTexture);
		}
	}
}

void UMediaPlateComponent::UnregisterWithMediaTextureTracker()
{
	// Remove out texture.
	if (MediaTextureTrackerObject != nullptr)
	{
		FMediaTextureTracker& MediaTextureTracker = FMediaTextureTracker::Get();
		for (UMediaTexture* MediaTexture : MediaTextures)
		{
			MediaTextureTracker.UnregisterTexture(MediaTextureTrackerObject, MediaTexture);
		}
	}
}

bool UMediaPlateComponent::PlayMediaSource(UMediaSource* InMediaSource, bool bInPlayOnOpen)
{
	bool bIsPlaying = false;

	if (InMediaSource != nullptr)
	{
		// Set media options.
		if (MediaPlayer != nullptr)
		{
			const bool bIsPlaylist = MediaPlateResource.GetResourceType() == EMediaPlateResourceType::Playlist;

			// Play the source.
			FMediaPlayerOptions Options;
			Options.SeekTime = FTimespan::FromSeconds(StartTime);
			Options.PlayOnOpen = bInPlayOnOpen ? EMediaPlayerOptionBooleanOverride::Enabled : EMediaPlayerOptionBooleanOverride::Disabled;
			Options.Loop = (bLoop && (bIsPlaylist == false)) ? EMediaPlayerOptionBooleanOverride::Enabled : EMediaPlayerOptionBooleanOverride::Disabled;
			Options.InternalCustomOptions.Emplace(MediaPlayerOptionValues::Environment(), MediaPlayerOptionValues::Environment_Preview());
			// Set cache settings.
			Options.InternalCustomOptions.Emplace(MediaPlayerOptionValues::ImgMediaSmartCacheEnabled(), FVariant(CacheSettings.bOverride));
			Options.InternalCustomOptions.Emplace(MediaPlayerOptionValues::ImgMediaSmartCacheTimeToLookAhead(), FVariant(CacheSettings.TimeToLookAhead));

			bIsPlaying = MediaPlayer->OpenSourceWithOptions(InMediaSource, Options);
			// Did we play anything?
			if (bIsPlaying)
			{
				TryActivateAspectRatioAuto();
			}
		}
	}

	return bIsPlaying;
}

void UMediaPlateComponent::TryActivateAspectRatioAuto()
{
	if (MediaPlayer != nullptr)
	{
		// Are we using automatic aspect ratio?
		if (IsAspectRatioAutoAllowed())
		{
			// Start the clock sink so we can tick.
			IMediaModule* MediaModule = FModuleManager::LoadModulePtr<IMediaModule>("Media");
			if (MediaModule != nullptr)
			{
				if (ClockSink.IsValid() == false)
				{
					ClockSink = MakeShared<FMediaComponentClockSink, ESPMode::ThreadSafe>(this);
				}
				MediaModule->GetClock().AddSink(ClockSink.ToSharedRef());
			}
		}
	}
}

bool UMediaPlateComponent::IsAspectRatioAutoAllowed()
{
	return ((bIsAspectRatioAuto) &&
		(VisibleMipsTilesCalculations == EMediaTextureVisibleMipsTiles::Plane));
}

float UMediaPlateComponent::GetAspectRatio()
{
	if (StaticMeshComponent != nullptr)
	{
		// Calculate aspect ratio from the scale.
		FVector Scale = StaticMeshComponent->GetRelativeScale3D();
		float AspectRatio = 0.0f;
		if (Scale.Z != 0.0f)
		{
			AspectRatio = static_cast<float>(Scale.Y / Scale.Z);
		}
		return AspectRatio;
	}

	return 0.0f;
}

void UMediaPlateComponent::SetAspectRatio(float AspectRatio)
{
	// Get the static mesh.
	if (StaticMeshComponent != nullptr)
	{
		// Update the scale.
		float Height = 1.0f;
		if (AspectRatio != 0.0f)
		{
			Height = 1.0f / AspectRatio;
		}
		const FVector Scale(1.0f, 1.0f, Height);
#if WITH_EDITOR
		// Avoid dirtying the package if scale is not modified.
		if (Scale != StaticMeshComponent->GetRelativeScale3D())
		{
			StaticMeshComponent->Modify();
		}
#endif // WITH_EDITOR
		StaticMeshComponent->SetRelativeScale3D(Scale);

		UpdateLetterboxes();
	}
}

void UMediaPlateComponent::SetLetterboxAspectRatio(float AspectRatio)
{
	LetterboxAspectRatio = FMath::Max(0.0f, AspectRatio);
	if (LetterboxAspectRatio == 0.0f)
	{
		RemoveLetterboxes();
	}
	else
	{
		AddLetterboxes();
	}

	UpdateLetterboxes();
}

void UMediaPlateComponent::SetNumberOfTextures(int32 NumTextures)
{
	// Make sure we don't have null elements.
	MediaTextures.Remove(nullptr);

	if (MediaTextures.Num() != NumTextures)
	{
		if (IsRegistered())
		{
			UnregisterWithMediaTextureTracker();
		}
		if (MediaTextures.Num() > NumTextures)
		{
			MediaTextures.RemoveAt(NumTextures, MediaTextures.Num() - NumTextures);
		}
		else
		{
			while (MediaTextures.Num() < NumTextures)
			{
				UMediaTexture* MediaTexture = NewObject<UMediaTexture>(this);
				MediaTexture->NewStyleOutput = true;
				MediaTextures.Add(MediaTexture);
			}
		}

		SetUpTextures(UE::MediaPlateComponent::ESetUpTexturesFlags::ForceUpdateResource);
		if (IsRegistered())
		{
			RegisterWithMediaTextureTracker();
		}
	}
}

void UMediaPlateComponent::TickOutput()
{
	if (ProxySetAspectRatio(MediaPlayer))
	{
		// No need to tick anymore.
		StopClockSink();
	}
}

float UMediaPlateComponent::GetProxyRate() const
{
	return CurrentRate;
}

bool UMediaPlateComponent::SetProxyRate(float Rate)
{
	CurrentRate = Rate;
	IntendedPlaybackState = Rate == 0.0f ? EPlaybackState::Paused : EPlaybackState::Playing;
	return MediaPlayer ? MediaPlayer->SetRate(Rate) : true;
}

bool UMediaPlateComponent::IsExternalControlAllowed()
{
	// Allow control if we are visible.
	return IsVisible();
}

const FMediaSourceCacheSettings& UMediaPlateComponent::GetCacheSettings() const
{
	return CacheSettings;
}

UMediaSource* UMediaPlateComponent::ProxyGetMediaSourceFromIndex(int32 Index) const
{
	return UE::MediaPlateComponent::GetMediaSource(GetMediaPlaylist(), Index);
}

void UMediaPlateComponent::IncreaseMediaTextureProxyCount(int32 InTextureIndex)
{
	int32& ProxyCount = MediaTextureProxyCount.FindOrAdd(InTextureIndex, 0);
	++ProxyCount;
}

void UMediaPlateComponent::DecreaseMediaTextureProxyCount(int32 InTextureIndex)
{
	if (int32* ProxyCount = MediaTextureProxyCount.Find(InTextureIndex))
	{
		--(*ProxyCount);

		if (*ProxyCount <= 0)
		{
			MediaTextureProxyCount.Remove(InTextureIndex);
		}
	}
}

bool UMediaPlateComponent::IsMediaTextureProxied(int32 InTextureIndex) const
{
	if (const int32* RefCount = MediaTextureProxyCount.Find(InTextureIndex))
	{
		return *RefCount > 0;
	}
	return false;
}

UMediaTexture* UMediaPlateComponent::ProxyGetMediaTexture(int32 LayerIndex, int32 TextureIndex)
{
	UMediaTexture* MediaTexture = GetMediaTexture(TextureIndex);
	if (MediaTexture != nullptr)
	{
		IncreaseMediaTextureProxyCount(TextureIndex);
		
		// Ensure internal media player is closed to not interfere with external one (audio in particular).
		// todo: revisit this when audio proxying is supported. It might be possible to be more permissive.
		if (IsMediaPlatePlaying())
		{
			Close();
		}
		
		SetNormalMode(false);
		if (TextureLayers.Num() < LayerIndex + 1)
		{
			TextureLayers.SetNum(LayerIndex + 1);
		}

		// Fill up an empty slot if there is one.
		bool bIsTextureSet = false;
		for (int32 Index = 0; Index < TextureLayers[LayerIndex].Textures.Num(); ++Index)
		{
			if (TextureLayers[LayerIndex].Textures[Index] < 0)
			{
				TextureLayers[LayerIndex].Textures[Index] = TextureIndex;
				bIsTextureSet = true;
				break;
			}
		}
		if (bIsTextureSet == false)
		{
			TextureLayers[LayerIndex].Textures.Add(TextureIndex);
		}

		UpdateTextureLayers();
	}

	return MediaTexture;
}

void UMediaPlateComponent::ProxyReleaseMediaTexture(int32 LayerIndex, int32 TextureIndex)
{
	DecreaseMediaTextureProxyCount(TextureIndex);
	
	ProxySetTextureBlend(LayerIndex, TextureIndex, 0.0f);

	if (LayerIndex < TextureLayers.Num())
	{
		for (int32 Index = 0; Index < TextureLayers[LayerIndex].Textures.Num(); ++Index)
		{
			if (TextureLayers[LayerIndex].Textures[Index] == TextureIndex)
			{
				TextureLayers[LayerIndex].Textures[Index] = -1;
				break;
			}
		}

		UpdateTextureLayers();
	}
}

bool UMediaPlateComponent::ProxySetAspectRatio(UMediaPlayer* InMediaPlayer)
{
	bool bIsDone = false;

	if (IsAspectRatioAutoAllowed())
	{
		// Is the player ready?
		if ((InMediaPlayer != nullptr) && (InMediaPlayer->IsClosed() == false) &&
			(InMediaPlayer->IsPreparing() == false))
		{
			FIntPoint VideoDim = InMediaPlayer->GetVideoTrackDimensions(INDEX_NONE, INDEX_NONE);
			if (VideoDim.Y != 0)
			{
				// Set aspect ratio.
				float AspectRatio = (float)VideoDim.X / (float)VideoDim.Y;
				SetAspectRatio(AspectRatio);
				bIsDone = true;
			}
		}
	}
	else
	{
		bIsDone = true;
	}

	return bIsDone;
}

void UMediaPlateComponent::ProxySetTextureBlend(int32 LayerIndex, int32 TextureIndex, float Blend)
{
	if (AMediaPlate* MediaPlate = GetOwner<AMediaPlate>())
	{
		if (UMaterialInterface* Material = MediaPlate->GetCurrentMaterial())
		{
			UMaterialInstanceDynamic* MID = Cast<UMaterialInstanceDynamic>(Material);
			if (MID != nullptr)
			{
				int32 MatNumLayers = MediaTextures.Num() / MatNumTexPerLayer;
				if (LayerIndex < TextureLayers.Num())
				{
					int32 MaterialLayerIndex = TextureLayers[LayerIndex].MaterialLayerIndex;
					if (MaterialLayerIndex < MatNumLayers)
					{
						const TArray<int32>& Layer = TextureLayers[LayerIndex].Textures;
						for (int32 LayerTexIndex = 0;
							(LayerTexIndex < MatNumTexPerLayer) && (LayerTexIndex < Layer.Num());
							LayerTexIndex++)
						{
							if (Layer[LayerTexIndex] == TextureIndex)
							{
								int32 MatTexIndex = MaterialLayerIndex * MatNumTexPerLayer
									+ LayerTexIndex;
								static const FString BaseBlendName = TEXT("Blend");
								FString BlendName = BaseBlendName;
								BlendName.AppendInt(MatTexIndex);
								MID->SetScalarParameterValue(FName(*BlendName), Blend);
								break;
							}
						}
					}
				}
			}
		}
	}
}

float UMediaPlateComponent::GetForwardRate(UMediaPlayer* MediaPlayer)
{
	float Rate = MediaPlayer->GetRate();

	if (Rate < 1.0f)
	{
		Rate = 1.0f;
	}

	return 2.0f * Rate;
}

float UMediaPlateComponent::GetReverseRate(UMediaPlayer* MediaPlayer)
{
	float Rate = MediaPlayer->GetRate();

	if (Rate > -1.0f)
	{
		return -1.0f;
	}

	return 2.0f * Rate;
}

bool UMediaPlateComponent::RestartPlayer()
{
	if (MediaPlayer != nullptr)
	{
		if (bIsMediaPlatePlaying && IntendedPlaybackState == EPlaybackState::Playing)
		{
			MediaPlayer->Close();
			Open();
			return true;
		}
	}
	return false;
}

void UMediaPlateComponent::StopClockSink()
{
	if (ClockSink.IsValid())
	{
		IMediaModule* MediaModule = FModuleManager::GetModulePtr<IMediaModule>("Media");
		if (MediaModule != nullptr)
		{
			MediaModule->GetClock().RemoveSink(ClockSink.ToSharedRef());
		}
	}
}

bool UMediaPlateComponent::IsVisible()
{
	bool bIsVisible = ((StaticMeshComponent != nullptr) && (StaticMeshComponent->ShouldRender()));

	if (bIsVisible && bPlayOnlyWhenVisible)
	{
		bIsVisible = GetOwner()->WasRecentlyRendered();
	}

	return bIsVisible;
}

void UMediaPlateComponent::ResumeWhenVisible()
{
	if (MediaPlayer != nullptr)
	{
		if (ActualPlaybackState == EPlaybackState::Paused)
		{
			// Should we be playing?
			if (CurrentRate != 0.0f && PendingPlaybackState == EPlaybackState::Unset)
			{
				IntendedPlaybackState = EPlaybackState::Resume;
			}
		}
		else if (bWantsToPlayWhenVisible)
		{
			if ((bResumeWhenOpened == false) &&
				(MediaPlayer->IsPreparing() == false) &&
				(ActualPlaybackState == EPlaybackState::Paused))
			{
				bResumeWhenOpened = true;
				bWantsToPlayWhenVisible = false;
				Open();
			}
		}
	}
}

FTimespan UMediaPlateComponent::GetResumeTime()
{
	FTimespan PlayerTime;
	if (MediaPlayer != nullptr)
	{
		PlayerTime = MediaPlayer->GetTime();
		if (TimeWhenPlaybackPaused > 0.0)
		{
			double CurrentTime = FApp::GetGameTime();
			double ElapsedTime = CurrentTime - TimeWhenPlaybackPaused;
			PlayerTime += FTimespan::FromSeconds(ElapsedTime);

			// Are we over the length of the media?
			FTimespan MediaDuration = MediaPlayer->GetDuration();
			if ((PlayerTime > MediaDuration) && (MediaDuration > FTimespan::Zero()))
			{
				const bool bIsPlaylist = MediaPlateResource.GetResourceType() == EMediaPlateResourceType::Playlist;
				if ((bLoop) && (bIsPlaylist == false))
				{
					PlayerTime %= MediaDuration;
				}
				else
				{
					// It wont play if we seek to the very end, so go back a little bit.
					PlayerTime = MediaDuration - FTimespan::FromSeconds(0.001f);
				}
			}
			TimeWhenPlaybackPaused = -1.0;
		}
	}
	return PlayerTime;
}

void UMediaPlateComponent::UpdateTicking()
{
	bool bEnableTick = bIsMediaPlatePlaying;
	PrimaryComponentTick.SetTickFunctionEnable(bEnableTick);
}


void UMediaPlateComponent::UpdateLetterboxes()
{
	float AspectRatio = GetAspectRatio();
	if ((AspectRatio <= LetterboxAspectRatio) || (LetterboxAspectRatio <= 0.0f))
	{
		for (const TObjectPtr<UStaticMeshComponent>& Letterbox : Letterboxes)
		{
			if (Letterbox != nullptr)
			{
				Letterbox->Modify();

				Letterbox->SetVisibility(false);
			}
		}
	}
	else if (AspectRatio > 0.0f)
	{
		float DefaultHeight = 50.0f;
		float VideoHeight = DefaultHeight / AspectRatio;
		float MaxHeight = DefaultHeight / LetterboxAspectRatio;

		float LetterboxHeight = (MaxHeight - VideoHeight) * 0.5f;
		LetterboxHeight = FMath::Max(LetterboxHeight, 0.0f);
		FVector Scale(1.0f, 1.0f, LetterboxHeight / DefaultHeight);

		FVector Location(0.0f, 0.0f, VideoHeight + LetterboxHeight);

		for (const TObjectPtr<UStaticMeshComponent>& Letterbox : Letterboxes)
		{
			if (Letterbox != nullptr)
			{
				Letterbox->Modify();
				Letterbox->SetVisibility(true);
				Letterbox->SetRelativeScale3D(Scale);
				Letterbox->SetRelativeLocation(Location);
				Location.Z = -Location.Z;
			}
		}
	}
}


void UMediaPlateComponent::AddLetterboxes()
{
	if (Letterboxes.Num() == 0)
	{
		AActor* Owner = GetOwner();
		if (Owner != nullptr)
		{
			UStaticMesh* Mesh = LoadObject<UStaticMesh>(NULL, TEXT("/MediaPlate/SM_MediaPlateScreen"), NULL, LOAD_None, NULL);
			UMaterial* Material = LoadObject<UMaterial>(NULL, TEXT("/MediaPlate/M_MediaPlate_Opaque"), NULL, LOAD_None, NULL);
			if ((Mesh != nullptr) && (Material != nullptr))
			{
				for (int32 Index = 0; Index < 2; ++Index)
				{
					UStaticMeshComponent* Letterbox = NewObject<UStaticMeshComponent>(Owner);
					if (Letterbox != nullptr)
					{
						Letterboxes.Add(Letterbox);
						Owner->Modify();
						Owner->AddInstanceComponent(Letterbox);
						Letterbox->OnComponentCreated();
						Letterbox->AttachToComponent(Owner->GetRootComponent(),
							FAttachmentTransformRules::KeepRelativeTransform);
						Letterbox->RegisterComponent();
						Letterbox->SetStaticMesh(Mesh);
						Letterbox->SetMaterial(0, Material);
						Letterbox->bCastStaticShadow = false;
						Letterbox->bCastDynamicShadow = false;
						Letterbox->SetVisibility(true);
					}
				}
			}
		}
	}
}

void UMediaPlateComponent::RemoveLetterboxes()
{
	for (const TObjectPtr<UStaticMeshComponent>& Letterbox : Letterboxes)
	{
		if (Letterbox != nullptr)
		{
			Letterbox->DestroyComponent();
		}
	}

	Letterboxes.Empty();
}

void UMediaPlateComponent::OnMediaOpened(FString DeviceUrl)
{
	if (bResumeWhenOpened)
	{
		bResumeWhenOpened = false;
		bIsMediaPlatePlaying = true;
		if (MediaPlayer != nullptr)
		{
			FTimespan PlayTime = GetResumeTime();
			MediaPlayer->Seek(PlayTime);
		}
	}
}

void UMediaPlateComponent::OnMediaEnd()
{
	StopClockSink();

	Next();
}

void UMediaPlateComponent::OnMediaResumed()
{
	ActualPlaybackState = EPlaybackState::Playing;
}

void UMediaPlateComponent::OnMediaSuspended()
{
	ActualPlaybackState = EPlaybackState::Paused;
}

void UMediaPlateComponent::SetUpTextures(UE::MediaPlateComponent::ESetUpTexturesFlags Flags)
{
	// Prevent media texture blackouts by only updating resource and material uniforms on relevant changes.
	bool bApplyMaterialUpdate = false;
	for (int32 MediaTextureIndex = 0; MediaTextureIndex < MediaTextures.Num(); ++MediaTextureIndex)
	{
		if (UMediaTexture* MediaTexture = MediaTextures[MediaTextureIndex])
		{
			bool bApplyTextureUpdate = false;

			if (MediaTexture->EnableGenMips != MediaTextureSettings.bEnableGenMips)
			{
				MediaTexture->EnableGenMips = MediaTextureSettings.bEnableGenMips;
				bApplyTextureUpdate = true;
			}

			if (FMath::IsNearlyEqual(MediaTexture->GetMipMapBias(), MipMapBias) == false)
			{
				MediaTexture->SetMipMapBias(MipMapBias);
				bApplyTextureUpdate = true;
				bApplyMaterialUpdate = true;
			}

			// Even if setting the player is allowed, we don't set the player if the texture is currently being proxied.
			if (EnumHasAllFlags(Flags, UE::MediaPlateComponent::ESetUpTexturesFlags::AllowSetPlayer)
				&& MediaTexture->GetMediaPlayer() != MediaPlayer.Get()
				&& !IsMediaTextureProxied(MediaTextureIndex))
			{
				MediaTexture->SetMediaPlayer(MediaPlayer);
				bApplyTextureUpdate = true;
			}

			if (bApplyTextureUpdate ||
				(EnumHasAllFlags(Flags, UE::MediaPlateComponent::ESetUpTexturesFlags::ForceUpdateResource)))
			{
				MediaTexture->UpdateResource();
			}
		}
	}

	if (bApplyMaterialUpdate)
	{
		if (AMediaPlate* MediaPlate = GetOwner<AMediaPlate>())
		{
			if (UMaterialInterface* Material = MediaPlate->GetCurrentMaterial())
			{
				Material->RecacheUniformExpressions(false);
			}
		}
	}
}


void UMediaPlateComponent::SetNormalMode(bool bInIsNormalMode)
{
#if WITH_EDITOR
	// Switching between normal mode and proxy mode should only be needed in the editor.
	if (bIsNormalMode != bInIsNormalMode)
	{
		bIsNormalMode = bInIsNormalMode;
		if (bIsNormalMode)
		{
			// Only want 1 texture.
			if (TextureLayers.Num() != 1)
			{
				TextureLayers.SetNum(1);
			}
			if (TextureLayers[0].Textures.Num() != 1)
			{
				TextureLayers[0].Textures.SetNum(1);
			}
			TextureLayers[0].Textures[0] = 0;
			UpdateTextureLayers();

			ProxySetTextureBlend(0, 0, 1.0f);
			MediaTextures[0]->SetMediaPlayer(MediaPlayer);
		}
		else
		{
			// Proxy will set these up.
			TextureLayers.Reset();
		}
	}
#endif // WITH_EDITOR
}

bool UMediaPlateComponent::IsNormalMode() const
{
#if WITH_EDITOR
	return bIsNormalMode;
#else
	return true;
#endif
}

bool UMediaPlateComponent::IsAnyMediaTextureProxied() const
{
	for (const TPair<int32, int32>& TextureProxyCount : MediaTextureProxyCount)
	{
		if (TextureProxyCount.Value > 0)
		{
			return true;
		}
	}
	return false;
}

void UMediaPlateComponent::UpdateTextureLayers()
{
	if (AMediaPlate* MediaPlate = GetOwner<AMediaPlate>())
	{
		static const FString BaseTextureName = TEXT("MediaTexture");

		if (UMaterialInterface* Material = MediaPlate->GetCurrentMaterial())
		{
			UMaterialInstanceDynamic* MID = Cast<UMaterialInstanceDynamic>(Material);
			if (MID != nullptr)
			{
				// Go through each layer.
				int32 MatNumLayers = MediaTextures.Num() / MatNumTexPerLayer;
				int32 NumLayers = TextureLayers.Num();
				int32 MaterialLayerIndex = 0;
				for (int32 LayerIndex = 0; LayerIndex < NumLayers; ++LayerIndex)
				{
					// Go through each texture in the layer.
					const TArray<int32>& Layer = TextureLayers[LayerIndex].Textures;
					int32 NumTex = FMath::Min(MatNumTexPerLayer, Layer.Num());
					TextureLayers[LayerIndex].MaterialLayerIndex = -1;
					for (int32 LayerTexIndex = 0; LayerTexIndex < NumTex; LayerTexIndex++)
					{
						// Set the texture in the material according to the layer data.
						int32 TextureIndex = Layer[LayerTexIndex];
						if (TextureIndex >= 0)
						{
							// Assign the next layer in the material to this layer.
							TextureLayers[LayerIndex].MaterialLayerIndex = MaterialLayerIndex;

							int32 MatTexIndex = MaterialLayerIndex * MatNumTexPerLayer +
								LayerTexIndex;
							FString TextureName = BaseTextureName;
							if (MatTexIndex != 0)
							{
								TextureName.AppendInt(MatTexIndex);
							}
							MID->SetTextureParameterValue(FName(*TextureName), MediaTextures[TextureIndex]);
						}
					}

					// Did we use this layer?
					if (TextureLayers[LayerIndex].MaterialLayerIndex != -1)
					{
						MaterialLayerIndex++;
						// Did we run out of layers in the material?
						if (MaterialLayerIndex >= MatNumLayers)
						{
							break;
						}
					}
				}
			}
		}

		UMaterialInterface* OverlayMaterial = MediaPlate->GetCurrentOverlayMaterial();
		if (OverlayMaterial != nullptr)
		{
			UMaterialInstanceDynamic* MID = Cast<UMaterialInstanceDynamic>(OverlayMaterial);
			if (MID != nullptr && !MediaTextures.IsEmpty())
			{

				MID->SetTextureParameterValue(FName(*BaseTextureName), MediaTextures[0]);
			}
		}
	}
}

#if WITH_EDITORONLY_DATA
void UMediaPlateComponent::InitializeMediaPlateResource()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UMediaPlaylist* Playlist = MediaPlaylist_DEPRECATED;

	if (!Playlist)
	{
		return;
	}

	const int32 PlaylistElementsNum = Playlist->Num();

	if (PlaylistElementsNum == 0)
	{
		return;
	}

	if (PlaylistElementsNum > 1)
	{
		// Playlist has more than one element, source type will be set to playlist
		SelectMediaPlaylistAsset(Playlist);
	}
	else
	{
		if (UMediaSource* MediaSource = Playlist->Get(0))
		{
			// If outer is this, source has been created from external file
			if (MediaSource->GetOuter() == this)
			{
				ExternalMediaSource = MediaSource;
				// Update external media path to match (as best we can).
				MediaPlateResource.LoadExternalMedia(UE::MediaPlateComponent::ExtractExternalMediaPath(ExternalMediaSource, this));
			}
			else
			{
				SelectMediaSourceAsset(MediaSource);
			}
		}
	}

	MediaPlaylist_DEPRECATED = nullptr;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}
#endif

void UMediaPlateComponent::RefreshMediaPlateResource()
{
	RefreshExternalMediaSource();
	RefreshActivePlaylist();

	const UMediaSource* SelectedMediaSource = UE::MediaPlateComponent::GetMediaSource(GetMediaPlaylist(), PlaylistIndex);

	// Remote Control may call PostEditChangeProperty more than once from various paths
	// with some delays causing Media Plate to close when it shouldn't. We mitigate that by
	// skipping the closing/restarting logic if the media source didn't change.
	if (MediaPlayer && SelectedMediaSource)
	{
		const UMediaSource* PlayerMediaSource = UE::MediaPlateComponent::GetMediaSource(MediaPlayer->GetPlaylist(), MediaPlayer->GetPlaylistIndex());
		if (SelectedMediaSource == PlayerMediaSource)
		{
			return;
		}

		// Only try to restart the player if there is a new selected media source.
		if (RestartPlayer())
		{
			return;
		}
	}

	constexpr EMediaPlateEventState State = EMediaPlateEventState::Close;
	SwitchStates(State);

	if (IMediaAssetsModule* MediaAssets = FModuleManager::LoadModulePtr<IMediaAssetsModule>("MediaAssets"))
	{
		MediaAssets->BroadcastOnMediaStateChangedEvent({GetOwner()->GetPathName()}, static_cast<uint8>(State));
	}

	// If the media source has changed, reset the associated media texture to clear the stale
	// video sample from the previous asset.
	// Note: if exposed to media track, the texture is not cleared if the track is evaluating, i.e.
	// the track has its own player associated to the texture and will not be affected by this.
	UE::MediaPlateComponent::ResetMediaTextureForPlayer(MediaPlayer, MediaTextures);
}

void UMediaPlateComponent::RefreshExternalMediaSource()
{
	if (MediaPlateResource.GetResourceType() == EMediaPlateResourceType::External)
	{
		ExternalMediaSource = UMediaSource::SpawnMediaSourceForString(MediaPlateResource.ExternalMediaPath, this);
	}
}

void UMediaPlateComponent::RefreshActivePlaylist()
{
	UMediaSource* SelectedMediaSource = nullptr;
	
	switch (MediaPlateResource.GetResourceType())
	{
	case EMediaPlateResourceType::Playlist:
		ActivePlaylist = MediaPlateResource.GetSourcePlaylist();
		return;

	case EMediaPlateResourceType::External:
		if (!ExternalMediaSource)
		{
			RefreshExternalMediaSource();
		}
		SelectedMediaSource = ExternalMediaSource;
		break;

	case EMediaPlateResourceType::Asset:
		SelectedMediaSource = MediaPlateResource.GetMediaAsset();
		break;

	default:
		break;
	}

	// Note: Must create a new playlist even if no media source is specified in
	// order to reset the previous media source.
	ActivePlaylist = NewObject<UMediaPlaylist>(this, NAME_None, RF_Transactional);
	
	if (SelectedMediaSource)
	{
		ActivePlaylist->Add(SelectedMediaSource);
	}
}

void UMediaPlateComponent::RefreshMediaSoundComponent()
{
	// Are we turning on audio?
	if (bEnableAudio)
	{
		// Create the component if missing.
		if (!SoundComponent)
		{
			SoundComponent = CreateMediaSoundComponent();
		}

		if (SoundComponent)
		{
			SoundComponent->SetMediaPlayer(MediaPlayer);

			if (!SoundComponent->IsRegistered())
			{
				// If called post undo: the component is marked as garbage and will not register again
				// unless we clear the garbage flag.
				SoundComponent->ClearGarbage();
				SoundComponent->bAutoRegister = true;	// Restore auto-register (if the component is restored from undo)
				SoundComponent->Initialize();
				SoundComponent->RegisterComponent();
			}
		}
	}
	else
	{
		// Note that removing the component loses all settings that where done to it.
		// This is why in the undo, we try to salvage the restored component rather than creating a new one.
		RemoveMediaSoundComponent();
	}
}

UMediaSoundComponent* UMediaPlateComponent::CreateMediaSoundComponent()
{
	// Media player must be valid.
	if (!MediaPlayer)
	{
		UE_LOGF(LogMediaPlate, Error, "Failed to create MediaSoundComponent: Missing Media Player.");
		return nullptr;
	}
	
	// Create a sound component.
	if (UMediaSoundComponent* NewSoundComponent = NewObject<UMediaSoundComponent>(this, NAME_None))
	{
		NewSoundComponent->bIsUISound = true;
		return NewSoundComponent;
	}
			
	UE_LOGF(LogMediaPlate, Error, "Failed to create MediaSoundComponent.");
	return nullptr;
}

void UMediaPlateComponent::RemoveMediaSoundComponent()
{
	if (SoundComponent != nullptr)
	{
		SoundComponent->UnregisterComponent();
		SoundComponent->SetMediaPlayer(nullptr);
		SoundComponent->UpdatePlayer();
		SoundComponent->DestroyComponent();
		SoundComponent->bAutoRegister = false;	// Don't let the component register again.
		SoundComponent = nullptr;
	}
}

#if WITH_EDITOR

void UMediaPlateComponent::SetVisibleMipsTilesCalculations(EMediaTextureVisibleMipsTiles InVisibleMipsTilesCalculations)
{
	VisibleMipsTilesCalculations = InVisibleMipsTilesCalculations;

	if (MediaTextureTrackerObject != nullptr)
	{
		MediaTextureTrackerObject->VisibleMipsTilesCalculations = VisibleMipsTilesCalculations;

		RestartPlayer();
	}
}

void UMediaPlateComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// Has bEnableAudio changed?
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(ThisClass, bEnableAudio))
	{
		RefreshMediaSoundComponent();
	}
	else if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(ThisClass, bPlayOnlyWhenVisible))
	{
		PlayOnlyWhenVisibleChanged();
	}
	else if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(ThisClass, CacheSettings))
	{
		RestartPlayer();
	}
	else if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(ThisClass, VisibleMipsTilesCalculations))
	{
		if (MediaTextureTrackerObject != nullptr)
		{
			MediaTextureTrackerObject->VisibleMipsTilesCalculations = VisibleMipsTilesCalculations;

			RestartPlayer();
		}
	}
	else if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(ThisClass, MipMapBias))
	{
		if (MediaTextureTrackerObject != nullptr)
		{
			MediaTextureTrackerObject->MipMapLODBias = MipMapBias;

			// Note: Media texture bias and material sampler automatically updated by UMediaPlateComponent::OnRegister().
		}
	}
	else if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(ThisClass, bEnableMipMapUpscaling)
		|| PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(ThisClass, MipLevelToUpscale))
	{
		if (MediaTextureTrackerObject != nullptr)
		{
			MediaTextureTrackerObject->MipLevelToUpscale = bEnableMipMapUpscaling ? MipLevelToUpscale : -1;
		}
	}
	else if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FMediaTextureResourceSettings, bEnableGenMips))
	{
		UE::MediaPlateComponent::ApplyMediaTextureMipGenProperties(MediaTextureSettings, MediaTextures);
	}
	else if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(ThisClass, MediaPlateResource))
	{
		RefreshMediaPlateResource();
	}
}

void UMediaPlateComponent::PostEditChangeChainProperty(FPropertyChangedChainEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(InPropertyChangedEvent);

	static const FName TypePropertyName = GET_MEMBER_NAME_CHECKED(FMediaPlateResource, Type);
	static const FName MediaAssetPropertyName = GET_MEMBER_NAME_CHECKED(FMediaPlateResource, MediaAsset);
	static const FName ExternalMediaPathPropertyName = GET_MEMBER_NAME_CHECKED(FMediaPlateResource, ExternalMediaPath);
	static const FName SourcePlaylistPropertyName = GET_MEMBER_NAME_CHECKED(FMediaPlateResource, SourcePlaylist);

	const FName MemberName = InPropertyChangedEvent.GetMemberPropertyName();
	
	if (MemberName == TypePropertyName
		|| MemberName == MediaAssetPropertyName
		|| MemberName == ExternalMediaPathPropertyName
		|| MemberName == SourcePlaylistPropertyName)
	{
		RefreshMediaPlateResource();
	}
}

void UMediaPlateComponent::PostEditUndo()
{
	Super::PostEditUndo();
	
	// Try to salvage the restored component. 
	RefreshMediaSoundComponent();

	RefreshMediaPlateResource();
}
#endif // WITH_EDITOR

void UMediaPlateComponent::SwitchStates(EMediaPlateEventState State)
{
	static const FTimespan TenSeconds = FTimespan::FromSeconds(10.f);

	switch (State)
	{
	case EMediaPlateEventState::Play:
		{
			Play();
		}
		break;
	case EMediaPlateEventState::PlayReverse:
		{
			PlayReverse();
		}
		break;
	case EMediaPlateEventState::Open:
		{
			Open();
		}
		break;
	case EMediaPlateEventState::Close:
		{
			Close();
		}
		break;
	case EMediaPlateEventState::Pause:
		{
			Pause();
		}
		break;
	case EMediaPlateEventState::Reverse:
		{
			if (MediaPlayer != nullptr)
			{
				MediaPlayer->SetRate(GetReverseRate(MediaPlayer));
			}
		}
		break;
	case EMediaPlateEventState::Forward:
		{
			if (MediaPlayer != nullptr)
			{
				MediaPlayer->SetRate(GetForwardRate(MediaPlayer));
			}
		}
		break;
	case EMediaPlateEventState::Rewind:
		{
			if (MediaPlayer != nullptr)
			{
				MediaPlayer->Rewind();
			}
		}
		break;
	case EMediaPlateEventState::Next:
		{
			Next();
		}
		break;
	case EMediaPlateEventState::Previous:
		{
			Previous();
		}
		break;
	case EMediaPlateEventState::JumpToEnd:
		{
			if (MediaPlayer != nullptr)
			{
				MediaPlayer->Seek(MediaPlayer->GetDuration());
			}
		}
		break;
	case EMediaPlateEventState::StepForward:
		{
			if (MediaPlayer != nullptr)
			{
				// TODO: Convert to underlying media player command when framework support is added.
				const float FrameRate = MediaPlayer->GetVideoTrackFrameRate(INDEX_NONE, INDEX_NONE);

				if (!FMath::IsNearlyZero(FrameRate))
				{
					MediaPlayer->Seek(MediaPlayer->GetTime() + FTimespan::FromSeconds(1.f / FrameRate));
				}
			}
		}
		break;
	case EMediaPlateEventState::StepBackward:
		{
			if (MediaPlayer != nullptr)
			{
				// TODO: Convert to underlying media player command when framework support is added.
				const float FrameRate = MediaPlayer->GetVideoTrackFrameRate(INDEX_NONE, INDEX_NONE);

				if (!FMath::IsNearlyZero(FrameRate))
				{
					MediaPlayer->Seek(MediaPlayer->GetTime() - FTimespan::FromSeconds(1.f / FrameRate));
				}
			}
		}
		break;
	case EMediaPlateEventState::MAX:
	default:
		checkNoEntry();
		break;
	}
}

#undef LOCTEXT_NAMESPACE
