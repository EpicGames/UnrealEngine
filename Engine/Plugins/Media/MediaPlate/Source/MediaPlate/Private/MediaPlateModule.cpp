// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaPlateModule.h"

#include "IMediaAssetsModule.h"
#include "MediaPlate.h"
#include "MediaPlateComponent.h"

DEFINE_LOG_CATEGORY(LogMediaPlate);

UMediaPlayer* FMediaPlateModule::GetMediaPlayer(UObject* Object, UObject*& PlayerProxy)
{
	UMediaPlayer* MediaPlayer = nullptr;

	// Is this a media plate?
	AMediaPlate* MediaPlate = Cast<AMediaPlate>(Object);
	if (MediaPlate != nullptr)
	{
		UMediaPlateComponent* MediaPlateComponent = MediaPlate->MediaPlateComponent;
		if (MediaPlateComponent != nullptr)
		{
			MediaPlayer = MediaPlateComponent->GetMediaPlayer();
			PlayerProxy = MediaPlateComponent;
		}
	}

	return MediaPlayer;
}

void FMediaPlateModule::StartupModule()
{
	IMediaAssetsModule* MediaAssetsModule = FModuleManager::LoadModulePtr<IMediaAssetsModule>("MediaAssets");
	if (MediaAssetsModule != nullptr)
	{
		GetPlayerFromObjectID = MediaAssetsModule->RegisterGetPlayerFromObject(IMediaAssetsModule::FOnGetPlayerFromObject::CreateRaw(this, &FMediaPlateModule::GetMediaPlayer));

#if WITH_EDITOR
		OnMediaPlateStateChangedHandle = MediaAssetsModule->RegisterOnMediaStateChangedEvent(IMediaAssetsModule::FMediaStateChangedDelegate::FDelegate::CreateRaw(this, &FMediaPlateModule::OnMediaPlateStateChanged));
#endif
	}

}

void FMediaPlateModule::ShutdownModule()
{
	if (GetPlayerFromObjectID != INDEX_NONE)
	{
		IMediaAssetsModule* MediaAssetsModule = FModuleManager::GetModulePtr<IMediaAssetsModule>("MediaAssets");
		if (MediaAssetsModule != nullptr)
		{
			MediaAssetsModule->UnregisterGetPlayerFromObject(GetPlayerFromObjectID);
			GetPlayerFromObjectID = INDEX_NONE;

#if WITH_EDITOR
			MediaAssetsModule->UnregisterOnMediaStateChangedEvent(OnMediaPlateStateChangedHandle);
#endif
		}
	}
}

#if WITH_EDITOR
void FMediaPlateModule::OnMediaPlateStateChanged(const TArray<FString>& InActorsPathNames, uint8 InEnumState, bool bRemoteBroadcast)
{
	// Button was pressed locally. This event will be handled in FMediaPlateCustomization.
	if (!bRemoteBroadcast)
	{
		return;
	}
	for (const FString& ActorPathName : InActorsPathNames)
	{
		AActor* MediaPlateActor = Cast<AActor>(StaticFindObject(AActor::StaticClass(), nullptr, *ActorPathName, EFindObjectFlags::None));
		if (MediaPlateActor)
		{
			TArray<UMediaPlateComponent*> MediaPlateComponents;
			MediaPlateActor->GetComponents<UMediaPlateComponent>(MediaPlateComponents);
			for (UMediaPlateComponent* MediaPlateComponent : MediaPlateComponents)
			{
				MediaPlateComponent->SwitchStates((EMediaPlateEventState)InEnumState);
			}
		}
	}
}
#endif

IMPLEMENT_MODULE(FMediaPlateModule, MediaPlate)
