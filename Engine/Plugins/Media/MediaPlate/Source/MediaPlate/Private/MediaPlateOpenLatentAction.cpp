// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaPlateOpenLatentAction.h"

#include "IMediaEventSink.h"
#include "MediaHelpers.h"
#include "MediaPlateComponent.h"
#include "MediaPlateModule.h"
#include "MediaPlayer.h"
#include "MediaTexture.h"

// Implementation references:
// - UMediaSourceRenderer: "wait on texture" part.
// - UMediaPlayer::OpenSourceLatent: player state management.

FMediaPlateOpenLatentAction::FMediaPlateOpenLatentAction(const FLatentActionInfo& InLatentInfo,
	UMediaPlateComponent* InMediaPlateComponent, float InWaitTimeout, bool bInWaitForTexture, bool& InOutSuccess)
	: ExecutionFunction(InLatentInfo.ExecutionFunction)
	, OutputLink(InLatentInfo.Linkage)
	, CallbackTarget(InLatentInfo.CallbackTarget)
	, MediaPlateComponentWeak(InMediaPlateComponent)
	, bWaitForTexture(bInWaitForTexture)
	, OutSuccess(InOutSuccess)
	, TimeRemaining(InWaitTimeout)
{
	if (InMediaPlateComponent)
	{
		InMediaPlateComponent->Open();

		if (UMediaPlayer* MediaPlayer = InMediaPlateComponent->GetMediaPlayer())
		{
			MediaPlayerWeak = MediaPlayer;
			MediaPlayer->OnMediaEvent().AddRaw(this, &FMediaPlateOpenLatentAction::OnMediaEvent);
			URL = MediaPlayer->GetUrl();

			if (bWaitForTexture)
			{
				// Reset the aspect ratio to be able to detect when a sample has been processed by the texture.
				if (UMediaTexture* Texture = InMediaPlateComponent->GetMediaTexture())
				{
					Texture->CurrentAspectRatio = 0.0f;
				}
			}
		}
		else
		{
			UE_LOGF(LogMediaPlate, Warning, "[%d] Media Plate Open Latent: Failed initial open: %ls", OutputLink, *URL);
			bSawError = true;
		}
	}
	else
	{
		UE_LOGF(LogMediaPlate, Warning, "[%d] Media Plate Open Latent: Failed initial open because no media source given", OutputLink);
		bSawError = true;
	}
}

FMediaPlateOpenLatentAction::~FMediaPlateOpenLatentAction()
{
	if (UMediaPlayer* MediaPlayer = MediaPlayerWeak.Get())
	{
		MediaPlayer->OnMediaEvent().RemoveAll(this);	
	}
}

void FMediaPlateOpenLatentAction::UpdateOperation(FLatentResponse& InResponse)
{
	if (bSawMediaOpenFailed)
	{
		UE_LOGF(LogMediaPlate, Warning, "[%d] Media Plate Open Latent: Saw media open failed event. %ls", OutputLink, *URL);
		FailedOperation(InResponse);
		return;
	}

	UMediaPlateComponent* MediaPlateComponent = MediaPlateComponentWeak.Get();

	if (!MediaPlateComponent)
	{
		UE_LOGF(LogMediaPlate, Warning, "[%d] Media Plate Open Latent: Media Plate Component object was deleted. %ls", OutputLink, *URL);
		FailedOperation(InResponse);
		return;
	}

	// Protect against internal media player being deleted or swapped out.
	const UMediaPlayer* CurrentMediaPlayer = MediaPlateComponent->GetMediaPlayer();
	UMediaPlayer* MediaPlayer = MediaPlayerWeak.Get();
	
	if (!MediaPlayer || !CurrentMediaPlayer || MediaPlayer != CurrentMediaPlayer)
	{
		UE_LOGF(LogMediaPlate, Warning, "[%d] Media Plate Open Latent: Media player object was deleted. %ls", OutputLink, *URL);
		FailedOperation(InResponse);
		return;
	}

	if (bSawError || MediaPlayer->HasError())
	{
		UE_LOGF(LogMediaPlate, Warning, "[%d] Media Plate Open Latent: Media player is in Error state. %ls", OutputLink, *URL);
		FailedOperation(InResponse);
		return;
	}

	if (MediaPlayer->IsClosed() || bSawMediaClosed)
	{
		UE_LOGF(LogMediaPlate, Warning, "[%d] Media Plate Open Latent: Media player is closed. %ls", OutputLink, *URL);
		FailedOperation(InResponse);
		return;
	}

	if (MediaPlayer->IsPreparing())
	{
		UE_LOGF(LogMediaPlate, Verbose, "[%d] Media Plate Open Latent: Is preparing ... %ls (Time out in %f seconds)", OutputLink, *URL,  TimeRemaining);
	}
	else if (MediaPlayer->IsReady())
	{
		if (!bSawIsReady)
		{
			// Show this only once when the state is reached.
			UE_LOGF(LogMediaPlate, Verbose, "[%d] Media Plate Open Latent: IsReady() ... %ls", OutputLink, *URL);
			bSawIsReady = true;
		}

		if (UpdateOperationPlayerReady(MediaPlayer, InResponse) == OperationUpdateResult::Completed)
		{
			return;
		}
	}
	else
	{
		UE_LOGF(LogMediaPlate, Verbose, "[%d] Media Plate Open Latent: Waiting for IsReady() ... %ls (Time out in %f seconds)", OutputLink, *URL, TimeRemaining);
	}

	// Update Timed out
	TimeRemaining -= InResponse.ElapsedTime();
	if (TimeRemaining <= 0.0f)
	{
		UE_LOGF(LogMediaPlate, Warning, "[%d] Media Plate Open Latent: Timed out. %ls", OutputLink, *URL);
		FailedOperation(InResponse);
		return;
	}
}

#if WITH_EDITOR
FString FMediaPlateOpenLatentAction::GetDescription() const
{
	return FString::Printf(TEXT("Media Plate Open Latent: %s %s (Time out in %f seconds)"), *GetStatusString(), *URL, TimeRemaining);
}
#endif

FString FMediaPlateOpenLatentAction::GetStatusString() const
{
	if (bSawMediaOpenFailed)
	{
		return TEXT("Media open failed event.");
	}

	UMediaPlateComponent* MediaPlateComponent = MediaPlateComponentWeak.Get();

	if (!MediaPlateComponent)
	{
		return TEXT("Media Plate Component object was deleted.");
	}

	const UMediaPlayer* CurrentMediaPlayer = MediaPlateComponent->GetMediaPlayer();
	const UMediaPlayer* MediaPlayer = MediaPlayerWeak.Get();

	if (!MediaPlayer || !CurrentMediaPlayer || CurrentMediaPlayer != MediaPlayer)
	{
		return TEXT("Media player object was deleted.");
	}

	if (bSawError || MediaPlayer->HasError())
	{
		return TEXT("Media player is in Error state.");
	}

	if (MediaPlayer->IsClosed())
	{
		return TEXT("Media player is closed.");
	}

	if (MediaPlayer->IsPreparing())
	{
		return TEXT("Is preparing ...");
	}

	if (!MediaPlayer->IsReady())
	{
		return TEXT("Waiting for IsReady() ...");
	}
	
	if (bWaitForTexture)
	{
		return TEXT("Is Ready - Waiting for Texture Render ...");
	}

	return FString::Printf(TEXT("Is Ready."));
}

FMediaPlateOpenLatentAction::OperationUpdateResult FMediaPlateOpenLatentAction::UpdateOperationPlayerReady(UMediaPlayer* InMediaPlayer, FLatentResponse& InResponse)
{
	if (bSawMediaOpened)
	{
		const FTimespan SeekTime = FTimespan::FromSeconds(MediaPlateComponentWeak->StartTime);

		// We need to issue a seek request to produce a sample if we are to wait for the texture.
		// Not all players (Protron, WMF) will produce a texture, unless there is a seek request.
		if (bWaitForTexture && !bSeekForTextureRequested)
		{
			bSeekForTextureRequested = true;
			InMediaPlayer->Seek(SeekTime);
		}
		
		if (!SeekTime.IsZero() || bSeekForTextureRequested)
		{
			if (bSawSeekCompleted)
			{
				return UpdateOperationConditionalWaitForTexture(InResponse);
			}
			
			if (SeekTime < FTimespan::FromSeconds(0) || SeekTime > InMediaPlayer->GetDuration())
			{
				UE_LOGF(LogMediaPlate, Warning, "[%d] Media Plate Open Latent: Media player seeking to time out of bounds. Seek: %ls, Duration: %ls, URL: %ls",
					OutputLink, *SeekTime.ToString(), *InMediaPlayer->GetDuration().ToString(), *URL);
				FailedOperation(InResponse);
				return OperationUpdateResult::Completed;
			}
			
			UE_LOGF(LogMediaPlate, Verbose, "[%d] Media Plate Open Latent: Waiting for seek completed event ... (Time out in %f seconds)", OutputLink, TimeRemaining);
		}
		else
		{
			return UpdateOperationConditionalWaitForTexture(InResponse);
		}
	}
	else
	{
		UE_LOGF(LogMediaPlate, Verbose, "[%d] Media Plate Open Latent: Waiting for opened event ... (Time out in %f seconds)", OutputLink, TimeRemaining);
	}

	return OperationUpdateResult::Continue;
}

FMediaPlateOpenLatentAction::OperationUpdateResult FMediaPlateOpenLatentAction::UpdateOperationConditionalWaitForTexture(FLatentResponse& InResponse) const
{
	if (bWaitForTexture)
	{
		// Is the texture ready?
		const UMediaTexture* Texture = MediaPlateComponentWeak->GetMediaTexture();
		// The aspect ratio will change when we have something.
		if (Texture && Texture->CurrentAspectRatio != 0.0f)
		{
			UE_LOGF(LogMediaPlate, Verbose, "[%d] Media Plate Open Latent: Triggering output pin after media texture ready. Success: %d, %ls", OutputLink, OutSuccess, *URL);
			CompleteOperation(InResponse);
			return OperationUpdateResult::Completed;
		}

		UE_LOGF(LogMediaPlate, Verbose, "[%d] Media Plate Open Latent: Waiting for texture ... (Time out in %f seconds)", OutputLink, TimeRemaining);	
		return OperationUpdateResult::Continue;
	}
	
	UE_LOGF(LogMediaPlate, Verbose, "[%d] Media Plate Open Latent: Triggering output pin after seek completed. Success: %d, %ls", OutputLink, OutSuccess, *URL);
	CompleteOperation(InResponse);
	return OperationUpdateResult::Completed;
}

void FMediaPlateOpenLatentAction::OnMediaEvent(EMediaEvent InEvent)
{
	UE_LOGF(LogMediaPlate, Verbose, "[%d] Media Plate Open Latent: Saw event: %ls", OutputLink, *MediaUtils::EventToString(InEvent));

	switch (InEvent)
	{
	case EMediaEvent::MediaOpened:
		bSawMediaOpened = true;
		break;
	case EMediaEvent::MediaOpenFailed:
		bSawMediaOpenFailed = true;
		break;
	case EMediaEvent::MediaClosed:
		bSawMediaClosed = true;
		break;
	case EMediaEvent::SeekCompleted:
		bSawSeekCompleted = true;
		break;

	default:
		break;
	}
}

void FMediaPlateOpenLatentAction::FailedOperation(FLatentResponse& InResponse) const
{
	OutSuccess = false;
	InResponse.FinishAndTriggerIf(true, ExecutionFunction, OutputLink, CallbackTarget);
}

void FMediaPlateOpenLatentAction::CompleteOperation(FLatentResponse& InResponse) const
{
	OutSuccess = true;
	InResponse.FinishAndTriggerIf(true, ExecutionFunction, OutputLink, CallbackTarget);
}