// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/LatentActionManager.h"
#include "IMediaEventSink.h"
#include "LatentActions.h"

class UMediaPlateComponent;
class UMediaPlayer;

/**
 * Pending Latent action to open a media plate component.
 * The difference with UMediaPlayer::OpenSourceLatent is that is also supports waiting
 * on the media texture to render a sample and therefore be truly ready to be displayed.
 * This can be used to hook in other animation systems to manage dynamic loading of media plates.
 */
class FMediaPlateOpenLatentAction : public FPendingLatentAction
{
public:
	FMediaPlateOpenLatentAction(const FLatentActionInfo& InLatentInfo,
		UMediaPlateComponent* InMediaPlateComponent, float InWaitTimeout, bool bInWaitForTexture, bool& InOutSuccess);
	
	virtual ~FMediaPlateOpenLatentAction() override;
	
	virtual void UpdateOperation(FLatentResponse& InResponse) override;

#if WITH_EDITOR
	/** Returns a human readable description of the latent operation's current state. */
	virtual FString GetDescription() const override;
#endif

private:
	/** Returns a human readable description of the latent operation's current state. */
	FString GetStatusString() const;

	/** Indicates if the operation is completed or needs to continue. */
	enum class OperationUpdateResult
	{
		Completed,
		Continue
	};

	/** Update Operation when the player is ready. */
	OperationUpdateResult UpdateOperationPlayerReady(UMediaPlayer* InMediaPlayer, FLatentResponse& InResponse);

	/** Update Operation when the player is ready and done seeking (if waiting for texture). */
	OperationUpdateResult UpdateOperationConditionalWaitForTexture(FLatentResponse& InResponse) const;

	/** Handler for UMediaPlayer::OnMediaEvent. */
	void OnMediaEvent(EMediaEvent InEvent);

	/** Ends the operation with failure state. */
	void FailedOperation(FLatentResponse& InResponse) const;

	/** Completes the operation with success. */
	void CompleteOperation(FLatentResponse& InResponse) const;

	/** Latent Action Info - The function to execute. */
	FName ExecutionFunction;
	
	/** Latent Action Info - The resume point within the function to execute. */
	int32 OutputLink;

	/** Latent Action Info - Object to execute the function on. */
	FWeakObjectPtr CallbackTarget;

	/** Media Plate Component the action is done on. */
	TWeakObjectPtr<UMediaPlateComponent> MediaPlateComponentWeak;

	/** Media Player the action is done on. */
	TWeakObjectPtr<UMediaPlayer> MediaPlayerWeak;

	/** Input parameter - If true the action will not be completed until the media texture has rendered a sample. */
	bool bWaitForTexture = false;

	/** Output parameter - Indicate if the operation completed successfully. */
	bool& OutSuccess;

	/** Keeps track of the remaining time, in seconds, before the operation times out. */
	float TimeRemaining = 10.0f;

	/** This is set to true if one of the callbacks indicate an error. */
	bool bSawError = false;

	/** Set to true if EMediaEvent::MediaOpened is received from the media player. */
	bool bSawMediaOpened = false;

	/** Set to true if EMediaEvent::MediaClosed is received from the media player. */
	bool bSawMediaClosed = false;

	/** Set to true if EMediaEvent::MediaOpenFailed is received from the media player. */
	bool bSawMediaOpenFailed = false;

	/** Set to true if EMediaEvent::SeekCompleted is received from the media player. */
	bool bSawSeekCompleted = false;

	/** Set to true if the media player has become ready. This is used to detect the state transition. */
	bool bSawIsReady = false;

	/** Set to true once the seek request has been issued. */
	bool bSeekForTextureRequested = false;

	/** Cache the media source's URL for display in log messages. */
	FString URL;
};
