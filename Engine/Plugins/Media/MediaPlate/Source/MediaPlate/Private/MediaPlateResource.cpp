// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaPlateResource.h"

#include "MediaPlaylist.h"
#include "MediaSource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaPlateResource)

void FMediaPlateResource::SetResourceType(EMediaPlateResourceType InType)
{
	Type = InType;
}

void FMediaPlateResource::SelectAsset(const UMediaSource* InMediaSource)
{
	SetResourceType(EMediaPlateResourceType::Asset);
	MediaAsset = const_cast<UMediaSource*>(InMediaSource);
}

void FMediaPlateResource::LoadExternalMedia(const FString& InFilePath)
{
	SetResourceType(EMediaPlateResourceType::External);
	ExternalMediaPath = InFilePath;
}

void FMediaPlateResource::SelectPlaylist(const UMediaPlaylist* InPlaylist)
{
	SetResourceType(EMediaPlateResourceType::Playlist);
	SourcePlaylist = const_cast<UMediaPlaylist*>(InPlaylist);
}

void FMediaPlateResource::Init(const FMediaPlateResource& InOther)
{
	if (!InOther.GetExternalMediaPath().IsEmpty())
	{
		ExternalMediaPath = FString(InOther.GetExternalMediaPath());
	}

	if (UMediaSource* OtherMediaAsset = InOther.GetMediaAsset())
	{
		MediaAsset = OtherMediaAsset;
	}

	if (UMediaPlaylist* OtherMediaPlaylist = InOther.GetSourcePlaylist())
	{
		SourcePlaylist = OtherMediaPlaylist;
	}

	Type = InOther.GetResourceType();
}

UMediaSource* FMediaPlateResource::GetMediaAsset() const
{
	return MediaAsset.LoadSynchronous();
}

UMediaPlaylist* FMediaPlateResource::GetSourcePlaylist() const
{
	return SourcePlaylist.LoadSynchronous();
}
