// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "Engine/AssetUserData.h"
#include "MediaPlate.generated.h"

#define UE_API MEDIAPLATE_API

class UMaterialInterface;
class UMaterialInstanceDynamic;
class UMediaPlateComponent;

/**
 * MediaPlate is an actor that can play and show media in the world.
 */
UCLASS(MinimalAPI)
class AMediaPlate : public AActor
{
	GENERATED_UCLASS_BODY()

public:
	//~ Begin UObject Interface.
	UE_API virtual void PostLoad() override;
	//~ End UObject Interface.

	//~ Begin AActor Interface
	UE_API virtual void PostActorCreated();
	UE_API virtual void PostRegisterAllComponents() override;
	UE_API virtual void BeginDestroy() override;
	//~ End AActor Interface

	UPROPERTY(Category = MediaPlate, VisibleAnywhere, BlueprintReadOnly)
	TObjectPtr<UMediaPlateComponent> MediaPlateComponent;

	/** Holds the mesh. */
	UPROPERTY(Category = MediaPlate, VisibleAnywhere, BlueprintReadOnly)
	TObjectPtr<UStaticMeshComponent> StaticMeshComponent;

	/* Set the holdout composite state. */
	UFUNCTION(BlueprintSetter)
	UE_API void SetHoldoutCompositeEnabled(bool bInEnabled);

	/* Get the holdout composite state. */
	UFUNCTION(BlueprintGetter)
	UE_API bool IsHoldoutCompositeEnabled() const;

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	/**
	 * Callback upon static mesh component change (see UMediaPlateAssetUserData).
	 */
	UE_API void OnStaticMeshChange();

	/**
	 * Call this to change the static mesh to use the default media plate material and reset the overlay material.
	 */
	UE_API void UseDefaultMaterial();

	/**
	 * Call this after changing the current material to set it up for media plate.
	 */
	UE_API void ApplyCurrentMaterial();
	
	/**
	 * Setup the material for media plate use. Automatically called by ApplyCurrentMaterial.
	 */
	UE_API void ApplyMaterial(UMaterialInterface* InMaterial);

	/**
	 * Setup the overlay material for media plate use. Automatically called by ApplyCurrentMaterial.
	 */
	UE_API void ApplyOverlayMaterial(UMaterialInterface* InOverlayMaterial);

	/**
	 * Sets up parameters (like the texture) that we use in the material.
	 */
	UE_API void SetMIDParameters(UMaterialInstanceDynamic* InMaterial);
	
	/** Get the last material assigned to the static mesh, at index 0. */
	UMaterialInterface* GetLastMaterial() const { return LastMaterial; }
#endif // WITH_EDITOR

	/** Get the current static mesh material, at index 0. */
	UE_API UMaterialInterface* GetCurrentMaterial() const;

	/** Get the current static mesh overlay material, nullptr otherwise. */
	UE_API UMaterialInterface* GetCurrentOverlayMaterial() const;

private:
	/** If true, the mesh is rendered separately and composited after post-processing (see HoldoutComposite plugin). Using mip generation with this setting is also recommended if the cost is acceptable, given that the composite bypasses temporal anti-aliasing. */
	UPROPERTY(EditAnywhere, BlueprintGetter = IsHoldoutCompositeEnabled, BlueprintSetter = SetHoldoutCompositeEnabled, Category = "Materials", meta = (AllowPrivateAccess = true))
	bool bEnableHoldoutComposite = false;

	/** Only enable registration with holdout composite subsytem if users have enabled holdout composite on the media plate.*/
	UE_API void ConditionallyEnableHoldoutComposite();

	/** Name for our media plate component. */
	static UE_API FLazyName MediaPlateComponentName;
	/** Name for the media texture parameter in the material. */
	static UE_API FLazyName MediaTextureName;

#if WITH_EDITOR
	UMaterialInterface* LastMaterial = nullptr;
	UMaterialInterface* LastOverlayMaterial = nullptr;

	/**
	 * Called before a level saves
	 */
	UE_API void OnPreSaveWorld(UWorld* InWorld, FObjectPreSaveContext ObjectSaveContext);

	/**
	 * Called after a level has saved.
	 */
	UE_API void OnPostSaveWorld(UWorld* InWorld, FObjectPostSaveContext ObjectSaveContext);

	/**
	 * Adds our asset user data to the static mesh component.
	 */
	UE_API void AddAssetUserData();

	/**
	 * Removes our asset user data from the static mesh component.
	 */
	UE_API void RemoveAssetUserData();

	/**
	 * Convenience function to apply create a material instance constant for media plate use.
	 */
	UE_API UMaterialInterface* CreateMaterialInstanceConstant(UMaterialInterface* InMaterial);

#endif // WITH_EDITOR
};

#undef UE_API
