// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/AssetUserData.h"
#include "MediaPlateAssetUserData.generated.h"

DECLARE_DELEGATE(FOnPostEditChangeOwner);

/**
 * AssetUserData for media plate.
 */
UCLASS()
class UMediaPlateAssetUserData : public UAssetUserData
{
	GENERATED_BODY()

public:
	/** Bind to this to get a callback when something is changed on the associated component. */
	FOnPostEditChangeOwner OnPostEditChangeOwner;
	
	//~ Begin UAssetUserData
	virtual void PostEditChangeOwner(const FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UAssetUserData
};
