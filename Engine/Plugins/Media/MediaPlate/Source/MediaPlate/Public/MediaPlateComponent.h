// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"
#include "MediaPlayerProxyInterface.h"
#include "MediaPlateResource.h"
#include "MediaSource.h"
#include "MediaTextureTracker.h"
#include "Misc/EnumClassFlags.h"

#include "MediaPlateComponent.generated.h"

#define UE_API MEDIAPLATE_API

class FMediaComponentClockSink;
class UMediaComponent;
class UMediaPlayer;
class UMediaPlaylist;
class UMediaSoundComponent;
class UMediaSource;
class UMediaTexture;

namespace UE::MediaPlateComponent
{
	enum class ESetUpTexturesFlags;
}

UENUM()
enum class EMediaPlateEventState : uint8
{
	Play,
	Open,
	Close,
	Pause,
	Reverse,
	Forward,
	Rewind,
	Next,
	Previous,
	PlayReverse,
	JumpToEnd,
	StepForward,
	StepBackward,
	MAX
};

/**
 * This struct is used to expose Media Texture settings via Media Plate Component and is a mirror of some
 * of the settings.
 */
USTRUCT()
struct FMediaTextureResourceSettings
{
	GENERATED_USTRUCT_BODY()

	/** Enable mips generation */
	UPROPERTY(EditAnywhere, Category = "MediaTexture", meta = (DisplayName = "Enable RealTime Mips"))
	bool bEnableGenMips = false;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.6, "Only 'Enable RealTime Mips' is needed now. When true, the full mip chain will be generated.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Only 'Enable RealTime Mips' is needed now. When true, the full mip chain will be generated."))
	uint8 CurrentNumMips_DEPRECATED = 1;
#endif

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FMediaTextureResourceSettings() = default;
	FMediaTextureResourceSettings(const FMediaTextureResourceSettings& InOther) = default;
	FMediaTextureResourceSettings(FMediaTextureResourceSettings&& InOther) = default;
	FMediaTextureResourceSettings& operator=(const FMediaTextureResourceSettings&) = default;
	FMediaTextureResourceSettings& operator=(FMediaTextureResourceSettings&&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
};


/**
 * This is a component for AMediaPlate that can play and show media in the world.
 */
UCLASS(MinimalAPI)
class UMediaPlateComponent : public UActorComponent,
	public IMediaPlayerProxyInterface
{
	GENERATED_UCLASS_BODY()

public:
	//~ UActorComponent interface.
#if WITH_EDITOR
	UE_API virtual void PostLoad() override;
#endif // WITH_EDITOR
	UE_API virtual void OnRegister() override;
	UE_API virtual void BeginPlay() override;
	UE_API virtual void EndPlay(const EEndPlayReason::Type InEndPlayReason) override;
	UE_API virtual void BeginDestroy() override;
	UE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	UE_API virtual void OnUnregister() override;
	UE_API virtual void OnComponentDestroyed(bool bDestroyingHierarchy) override;
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& InPropertyChangedEvent) override;
	UE_API virtual void PostEditUndo() override;
#endif // WITH_EDITOR

	/**
	 * Call this get our media player.
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlateComponent")
	UE_API UMediaPlayer* GetMediaPlayer();

	/**
	 * Call this get our media texture.
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlateComponent")
	UE_API UMediaTexture* GetMediaTexture(int32 Index = 0);

	/**
	 * Indicates if switching to the given event state (open, play, etc) is currently allowed by
	 * the backend.
	 * @param InRequestEventState Requested event state to switch to (Open, Play, etc)
	 * @return true if the state switch is allowed, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlateComponent")
	UE_API bool IsEventStateChangeAllowed(EMediaPlateEventState InRequestEventState) const;

	/**
	 * Call this to open the media.
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlateComponent")
	UE_API void Open();

	/**
	 * Open the media using a latent action.
	 *
	 * @param InTimeout Wait timeout in seconds
	 * @param bInWaitForTexture Wait for the media texture to have rendered a sample.
	 * @param bOutSuccess The media was opened successfully.
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlateComponent", meta = (Latent, LatentInfo="InLatentInfo", WorldContext="InWorldContextObject", InTimeout="10.0f", bInWaitForTexture="true"))	
	UE_API void OpenLatent(const UObject* InWorldContextObject, struct FLatentActionInfo InLatentInfo, float InTimeout, bool bInWaitForTexture, bool& bOutSuccess);
	
	/**
	 * Play the next item in the playlist.
	 *
	 * returns	True if it played something.
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlateComponent")
	UE_API bool Next();

	/**
	 * Call this to start playing at a rate of 1.0 (forward at normal speed)
	 * Open must be called before this.
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlateComponent")
	UE_API void Play();

	/**
	 * Call this to start playing at a rate of -1.0 (reverse at normal speed)
	 * Open must be called before this.
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlateComponent")
	UE_API void PlayReverse();

	/**
	 * Call this to pause playback.
	 * Play can be called to resume playback.
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlateComponent")
	UE_API void Pause();

	/**
	 * Play the previous item in the playlist.
	 *
	 * returns	True if it played something.
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlateComponent")
	UE_API bool Previous();

	/**
	 * Rewinds the media to the beginning.
	 *
	 * This is the same as seeking to zero time.
	 *
	 * @return				True if rewinding, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlateComponent")
	UE_API bool Rewind();

	/**
	 * Call this to seek to the specified playback time.
	 *
	 * @param Time			Time to seek to.
	 * @return				True on success, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlateComponent")
	UE_API bool Seek(const FTimespan& Time);

	/**
	 * Call this to close the media.
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlateComponent")
	UE_API void Close();

	/**
	 * Call this to see if the media plate is playing.
	 */
	UFUNCTION(BlueprintGetter)
	bool IsMediaPlatePlaying() const { return bIsMediaPlatePlaying; }

	/**
	 * Call this to see if we want to loop.
	 */
	UFUNCTION(BlueprintGetter)
	UE_API bool GetLoop();

	/**
	 * Call this enable/disable looping.
	 */
	UFUNCTION(BlueprintSetter)
	UE_API void SetLoop(bool bInLoop);

	/**
	 * Get the currently active Media Playlist
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlateComponent")
	UE_API UMediaPlaylist* GetMediaPlaylist() const;

	/**
	 * Get the currently active Media Source.
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlateComponent")
	UE_API UMediaSource* GetSelectedMediaSource() const;

	/**
	 * @brief Select the external media file (non-UE asset) to be opened.
	 * @param InFilePath non-UFS path (absolute or relative to executable) to the media file.
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlateComponent")
	UE_API void SelectExternalMedia(const FString& InFilePath);
	
	/**
	 * @brief Select the media source asset to be opened. 
	 * @param InMediaSource media source asset to open.
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlateComponent")
	UE_API void SelectMediaSourceAsset(const UMediaSource* InMediaSource);

	/**
	 * @brief Select the media playlist asset to be opened. 
	 * @param InMediaPlaylist media playlist asset to open.
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlateComponent")
	UE_API void SelectMediaPlaylistAsset(const UMediaPlaylist* InMediaPlaylist);
	
	/**
	 * Update Media Player Resource. This will also refresh Playlist accordingly.
	 */
	UE_API void SetMediaPlateResource(const FMediaPlateResource& InMediaPlayerResource);

	/** If set then play when opening the media. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Control")
	bool bPlayOnOpen = true;

	/** If set then start playing when this object is active. */
	UPROPERTY(EditAnywhere, Category = "Control")
	bool bAutoPlay = true;

	/** If set then enable audio. */
	UPROPERTY(EditAnywhere, Category = "Control")
	bool bEnableAudio = false;

	/** What time to start playing from (in seconds). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Control", meta = (ClampMin = "0.0"))
	float StartTime = 0.0f;

	/** Holds the component to play sound. */
	UPROPERTY(EditAnywhere, Category = "Advanced", meta = (DisplayName = "Audio Component"))
	TObjectPtr<UMediaSoundComponent> SoundComponent;

	/** Holds the component for the mesh. */
	UPROPERTY(EditAnywhere, Category = "Advanced")
	TObjectPtr<UStaticMeshComponent> StaticMeshComponent;

	/** Holds the component for the mesh. */
	UPROPERTY(EditAnywhere, Category = "Advanced|Other")
	TArray<TObjectPtr<UStaticMeshComponent>> Letterboxes;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.5, "Use MediaPlateResource instead")
	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage="Use MediaPlateResource instead"))
	TObjectPtr<UMediaPlaylist> MediaPlaylist_DEPRECATED;
#endif

	/** Which media source is used to populate the media playlist */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="MediaPlate", meta=(AllowPrivateAccess="true"), Setter=SetMediaPlateResource)
	FMediaPlateResource MediaPlateResource;

	/** The current index of the source in the play list being played. */
	UPROPERTY(BlueprintReadWrite, Category = "MediaPlate")
	int32 PlaylistIndex = 0;

	/** Override the default cache settings. */
	UPROPERTY(EditAnywhere, Category = "Cache", meta = (DisplayName = "Cache", ShowOnlyInnerProperties))
	FMediaSourceCacheSettings CacheSettings;

	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlateComponent")
	UE_API void SetEnableAudio(bool bInEnableAudio);

	/** Set the arc size in degrees used for visible mips and tiles calculations, specific to the sphere. */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlateComponent")
	UE_API void SetMeshRange(FVector2D InMeshRange);

	/** Return the arc size in degrees used for visible mips and tiles calculations, specific to the sphere. */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlateComponent")
	FVector2D GetMeshRange() const { return MeshRange; }

	/** Call this to set bPlayOnlyWhenVisible. */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlateComponent")
	UE_API void SetPlayOnlyWhenVisible(bool bInPlayOnlyWhenVisible);

	/**
	 * Call this to get the aspect ratio of the mesh.
	 */
	UE_API float GetAspectRatio();

	/**
	 * Call this to set the aspect ratio of the mesh.
	 */
	UE_API void SetAspectRatio(float AspectRatio);

	/**
	 * Gets whether automatic aspect ratio is enabled.
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlateComponent")
	bool GetIsAspectRatioAuto() const { return bIsAspectRatioAuto; }

	/**
	 * Sets whether automatic aspect ratio is enabled.
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlateComponent")
	UE_API void SetIsAspectRatioAuto(bool bInIsAspectRatioAuto);

	/**
	 * Call this to get the aspect ratio of the screen.
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlateComponent")
	float GetLetterboxAspectRatio() { return LetterboxAspectRatio; }

	/**
	 * Call this to set the aspect ratio of the screen.
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlateComponent")
	UE_API void SetLetterboxAspectRatio(float AspectRatio);

	/**
	 * Call this to see if this plate wants to play when it becomes visible.
	 */
	bool GetWantsToPlayWhenVisible() const { return bWantsToPlayWhenVisible; }

	/**
	 * Called from AMediaPlate to set how many media textures the material needs.
	 */
	UE_API void SetNumberOfTextures(int32 NumTextures);

#if WITH_EDITOR
	/**
	 * Call this to get the mip tile calculations mesh mode.
	 */
	EMediaTextureVisibleMipsTiles GetVisibleMipsTilesCalculations() const { return VisibleMipsTilesCalculations; }
	/**
	 * Call this to set the mip tile calculations mesh mode. (Note: restarts playback to apply changes.)
	 */
	UE_API void SetVisibleMipsTilesCalculations(EMediaTextureVisibleMipsTiles InVisibleMipsTilesCalculations);
#endif

	/**
	 * Called whenever a button was pressed locally or on a remote endpoint.
	 */
	UE_API void SwitchStates(EMediaPlateEventState State);

	/**
	 * Called from the media clock.
	 */
	UE_API void TickOutput();

	//~ IMediaPlayerProxyInterface.
	UE_API virtual float GetProxyRate() const override;
	UE_API virtual bool SetProxyRate(float Rate) override;
	UE_API virtual bool IsExternalControlAllowed() override;
	UE_API virtual const FMediaSourceCacheSettings& GetCacheSettings() const override;
	UE_API virtual UMediaSource* ProxyGetMediaSourceFromIndex(int32 Index) const override;
	UE_API virtual UMediaTexture* ProxyGetMediaTexture(int32 LayerIndex, int32 TextureIndex) override;
	UE_API virtual void ProxyReleaseMediaTexture(int32 LayerIndex, int32 TextureIndex) override;
	UE_API virtual bool ProxySetAspectRatio(UMediaPlayer* InMediaPlayer) override;
	UE_API virtual void ProxySetTextureBlend(int32 LayerIndex, int32 TextureIndex, float Blend) override;

public:
	/**
	 * Get the rate to use when we press the forward button.
	 */
	static UE_API float GetForwardRate(UMediaPlayer* MediaPlayer);

	/**
	 * Get the rate to use when we press the reverse button.
	 */
	static UE_API float GetReverseRate(UMediaPlayer* MediaPlayer);

private:
	/**
	 * Adds our media texture to the media texture tracker.
	 */
	UE_API void RegisterWithMediaTextureTracker();
	/**
	 * Removes our texture from the media texture tracker.
	 */
	UE_API void UnregisterWithMediaTextureTracker();

	/**
	 * Should be called when bPlayOnlyWhenVisible changes.
	 */
	UE_API void PlayOnlyWhenVisibleChanged();

	UE_API bool RestartPlayer();

	/**
	 * If true, then we want the media plate to play.
	 * Note that this could be true, but the player is not actually playing because
	 * bPlayOnlyWhenVisible = true and the plate is not visible.
	 */
	UPROPERTY(Blueprintgetter = IsMediaPlatePlaying, Category = "MediaPlate", meta = (AllowPrivateAccess = true))
	bool bIsMediaPlatePlaying = false;

	/** Desired rate of play that we want. */
	float CurrentRate = 0.0f;

	enum class EPlaybackState
	{
		Unset,
		Paused,
		Playing,
		Resume
	};
	/** State transitions. */
	EPlaybackState IntendedPlaybackState = EPlaybackState::Unset;
	EPlaybackState PendingPlaybackState = EPlaybackState::Unset;
	EPlaybackState ActualPlaybackState = EPlaybackState::Unset;


	/** If true then only allow playback when the media plate is visible. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Control", meta = (AllowPrivateAccess = true))
	bool bPlayOnlyWhenVisible = false;

	/** If set then loop when we reach the end. */
	UPROPERTY(EditAnywhere, Blueprintgetter = GetLoop, BlueprintSetter = SetLoop, Category = "Control", meta = (AllowPrivateAccess = true))
	bool bLoop = true;

	/** Visible mips and tiles calculation mode for the supported mesh types in MediaPlate. (Player restart on change.) */
	UPROPERTY(EditAnywhere, Category = "EXR Tiles & Mips", meta = (DisplayName = "Visible Tiles & Mips Logic", AllowPrivateAccess = true))
	EMediaTextureVisibleMipsTiles VisibleMipsTilesCalculations;

	/** Media texture mip map bias shared between the (image sequence) loader and the media texture sampler. */
	UPROPERTY(EditAnywhere, Category = "EXR Tiles & Mips", meta = (DisplayName = "Mips Bias", AllowPrivateAccess = true, UIMin = "-16.0", UIMax = "15.99"))
	float MipMapBias = 0.0f;

	/** If true then set the aspect ratio automatically based on the media. */
	UPROPERTY(Blueprintgetter = GetIsAspectRatioAuto, BlueprintSetter = SetIsAspectRatioAuto, Category = "MediaPlate", meta = (AllowPrivateAccess = true))
	bool bIsAspectRatioAuto = true;

	/** If true then enable the use of MipLevelToUpscale as defined below. */
	UPROPERTY(EditAnywhere, Category = "EXR Tiles & Mips", meta = (DisplayName = "Enable Mip Upscaling", AllowPrivateAccess = true))
	bool bEnableMipMapUpscaling = false;

	/* With exr playback, upscale into lower quality mips from this specified level. All levels including and above the specified value will be fully read. */
	UPROPERTY(EditAnywhere, Category = "EXR Tiles & Mips", meta = (DisplayName = "Upscale Mip Level", EditCondition = "bEnableMipMapUpscaling", EditConditionHides, AllowPrivateAccess = true, UIMin = "0", UIMax = "16"))
	int32 MipLevelToUpscale = 16;

	/** If true then Media Plate will attempt to load and upscale lower quality mips and display those at the poles (Sphere object only). */
	UPROPERTY(EditAnywhere, Category = "EXR Tiles & Mips", meta = (DisplayName = "Adaptive Pole Mip Upscale", EditCondition = "VisibleMipsTilesCalculations == EMediaTextureVisibleMipsTiles::Sphere", EditConditionHides, AllowPrivateAccess = true))
	bool bAdaptivePoleMipUpscaling = true;

	/** If > 0, then this is the aspect ratio of our screen and
	 * letterboxes will be added if the media is smaller than the screen. */
	UPROPERTY()
	float LetterboxAspectRatio = 0.0f;

	/** Number of textures we have per layer in the material. */
	const int32 MatNumTexPerLayer = 2;

	UPROPERTY()
	FVector2D MeshRange = FVector2D(360.0f, 180.0f);

	/** Name for our media component. */
	static UE_API FLazyName MediaComponentName;
	/** Name for our playlist. */
	static UE_API FLazyName MediaPlaylistName;

#if WITH_EDITORONLY_DATA
	/** Superseded by MediaTextures. */
	UPROPERTY(Instanced)
	TObjectPtr<UMediaTexture> MediaTexture_DEPRECATED;
#endif // WITH_EDITORONLY_DATA

	/** Holds the media textures. */
	UPROPERTY(Instanced)
	TArray<TObjectPtr<UMediaTexture>> MediaTextures;

	/** Exposes Media Texture settings via Media Plate component. */
	UPROPERTY(EditAnywhere, Category = "MediaTexture", meta = (ShowOnlyInnerProperties))
	FMediaTextureResourceSettings MediaTextureSettings;

	/** This component's media player */
	UPROPERTY(Instanced)
	TObjectPtr<UMediaPlayer> MediaPlayer;

	/** Embedded Media Source loaded from external path specified in MediaPlateResource. */
	UPROPERTY(Instanced)
	TObjectPtr<UMediaSource> ExternalMediaSource;

	/**
	 * Currently running Playlist.
	 * It can either be a stand alone or embedded asset depending on the media plate resource type. 
	 */
	UPROPERTY(Instanced)
	TObjectPtr<UMediaPlaylist> ActivePlaylist;
	
	/** Info representing this object. */
	TSharedPtr<FMediaTextureTrackerObject, ESPMode::ThreadSafe> MediaTextureTrackerObject;
	/** Our media clock sink. */
	TSharedPtr<FMediaComponentClockSink, ESPMode::ThreadSafe> ClockSink;
	/** Game time when we paused playback. */
	double TimeWhenPlaybackPaused = -1.0;
	/** True if our media should be playing when visible. */
	bool bWantsToPlayWhenVisible = false;
	/** True if we should resume where we left off when we open the media. */
	bool bResumeWhenOpened = false;

#if WITH_EDITOR
	/** True if we are in normal mode (as opposed to proxy mode). */
	bool bIsNormalMode = false;
#endif

	/**
	 * Contains all of our layers.
	 * Each layer contains which textures it has.
	 * int32 is an index into MediaTextures.
	 * -1 signifies no entry.
	 */
	struct Layer
	{
		/** The layer in the material that this layer uses. */
		int32 MaterialLayerIndex;
		/** List of textures in this layer. */
		TArray<int32> Textures;

		Layer() : MaterialLayerIndex(0) {};
	};
	TArray<Layer> TextureLayers;

	/**
	 * Map of TextureIndex to proxy count.
	 * 
	 * This accurately reflects the set of textures that are actively being used externally
	 * through the IMediaPlayerProxyInterface's Get/ReleaseMediaTexture functions.
	 * Given that Get/Release calls can be out of order depending on the playback or scrubbing direction,
	 * we need to keep track of this state using a ref counting method. Some texture can temporarily be
	 * referenced by 2 (or more) sections (even if not actually used internally).
	 */
	TMap<int32, int32> MediaTextureProxyCount;

	/** Called when the given media texture starts being proxied by a media section to increase the proxy count. */ 
	UE_API void IncreaseMediaTextureProxyCount(int32 InTextureIndex);

	/** Called when the given media texture stops being proxied by a media section to decrease the proxy count. */ 
	UE_API void DecreaseMediaTextureProxyCount(int32 InTextureIndex);

	/** Determines if the given texture index is being proxied. */
	UE_API bool IsMediaTextureProxied(int32 InTextureIndex) const;

	/**
	 * Plays a media source.
	 *
	 * @param	InMediaSource		Media source to play.
	 * @param	bInPlayOnOpen		True to play, false to just open.
	 * @return	True if we played anything.
	 */
	UE_API bool PlayMediaSource(UMediaSource* InMediaSource, bool bInPlayOnOpen);

	/**
	 * If the player is currently active, then this will set the aspect ratio
	 * according to the media.
	 */
	UE_API void TryActivateAspectRatioAuto();

	/**
	 * Returns true if auto aspect ratio is enabled and our mesh supports this (e.g. planar).
	 */
	UE_API bool IsAspectRatioAutoAllowed();

	/**
	 * Stops the clock sink so we no longer tick.
	 */
	UE_API void StopClockSink();

	/**
	 * Call this to see if this media plate is visible.
	 */
	UE_API bool IsVisible();

	/**
	 * Call this to resume playback when the media plate is visible.
	 */
	UE_API void ResumeWhenVisible();

	/**
	 * Returns the time to seek to when resuming playback.
	 */
	UE_API FTimespan GetResumeTime();

	/**
	 * Updates if we should tick or not based on current state.
	 */
	UE_API void UpdateTicking();

	/**
	 * Updates letterboxes based on the current state.
	 */
	UE_API void UpdateLetterboxes();

	/**
	 * Adds ability to have letterboxes.
	 */
	UE_API void AddLetterboxes();

	/**
	 * Removes ability to have letterboxes.
	 */
	UE_API void RemoveLetterboxes();

	/**
	 * Called by the media player when the media opens.
	 */
	UFUNCTION()
	UE_API void OnMediaOpened(FString DeviceUrl);

	/**
	 * Called by the media player when the video ends.
	 */
	UFUNCTION()
	UE_API void OnMediaEnd();

	/**
	 * Called by the media player when the video resumes.
	 */
	UFUNCTION()
	UE_API void OnMediaResumed();

	/**
	 * Called by the media player when the video pauses.
	 */
	UFUNCTION()
	UE_API void OnMediaSuspended();

	/**
	 * Sets up the textures we have.
	 */
	UE_API void SetUpTextures(UE::MediaPlateComponent::ESetUpTexturesFlags Flags);

	/**
	 * Sets either normal mode or proxy mode for something like Sequencer.
	 */
	UE_API void SetNormalMode(bool bInIsNormalMode);

	/**
	 * Returns true if normal mode of false if proxy mode.
	 */
	UE_API bool IsNormalMode() const;

	/**
	 * @brief Indicates if any media texture is currently being proxied by sequencer.
	 * @return true if at least one media texture is in use by sequencer.
	 */
	UE_API bool IsAnyMediaTextureProxied() const;

	/**
	 * Sets textures in our material according to the layer assignments.
	 */
	UE_API void UpdateTextureLayers();

#if WITH_EDITORONLY_DATA
	/**
	 * If Media Player Resource is not initialized yet (e.g. older assets)
	 * this function will initialize it with the proper data.
	 * Logic is the following: if media playlist has more than one entry, then the type will be set to Playlist.
	 * If playlist has only one entry, then the function will check for that entry Outer.
	 * If the Outer is this media plate component, then type will be set to External, otherwise to Asset.
	 */
	UE_API void InitializeMediaPlateResource();
#endif

	/**
	 * Can be called after updating MediaPlayerResource, to ensure the Playlist and player state are up to date
	 */
	UE_API void RefreshMediaPlateResource();

	/**
	 * If the current resource type is external path, this will load a new media source from that path.
	 */
	UE_API void RefreshExternalMediaSource();
	
	/**
	 * Refresh the ActivePlaylist from the current configuration of MediaPlateResource.
	 */
	UE_API void RefreshActivePlaylist();
	
	/**
	 * Refresh the media sound component according to the bEnableAudio variable.
	 */
	UE_API void RefreshMediaSoundComponent();

	/**
	 * Create the media sound component with initial setup (but not registered).
	 * @return created component on success, nullptr on failure.
	 */
	UE_API UMediaSoundComponent* CreateMediaSoundComponent();

	/**
	 * Removes the audio component.
	 */
	UE_API void RemoveMediaSoundComponent();
};

#undef UE_API
