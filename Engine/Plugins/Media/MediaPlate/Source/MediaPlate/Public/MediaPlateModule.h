// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Logging/LogMacros.h"
#include "Modules/ModuleManager.h"

#define UE_API MEDIAPLATE_API

class AMediaPlate;
class IMediaPlayerProxy;
class UMaterialInterface;
class UMediaPlayer;
class UObject;

/** Log category for this module. */
DECLARE_LOG_CATEGORY_EXTERN(LogMediaPlate, Log, All);

/**
 * Callback when a media plate applies a material to itself.
 * Set bCanModify to false if you do not want the media plate to modify the material so it can show
 * the media plate's texture.
 */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnMediaPlateApplyMaterial, UMaterialInterface*, AMediaPlate*, bool& /*bCanNodify*/)

class FMediaPlateModule : public IModuleInterface
{
public:
	/**
	 * Call this to get the media player from a media plate object.
	 */
	UE_API UMediaPlayer* GetMediaPlayer(UObject* Object, UObject*& PlayerProxy);

	/** IModuleInterface implementation */
	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;

	/** Add to this if you want a callback when a media plate applies a material to itself. */
	FOnMediaPlateApplyMaterial OnMediaPlateApplyMaterial;

#if WITH_EDITOR
	/**
	* Called when Media playback state changed.
	*/
	UE_API void OnMediaPlateStateChanged(const TArray<FString>& InActorsPathNames, uint8 InEnumState, bool bRemoteBroadcast);
#endif
private:
	/** ID for our delegate. */
	int32 GetPlayerFromObjectID = INDEX_NONE;

#if WITH_EDITOR
	/** Handle for the subscribed event */
	FDelegateHandle OnMediaPlateStateChangedHandle;
#endif
};

#undef UE_API
