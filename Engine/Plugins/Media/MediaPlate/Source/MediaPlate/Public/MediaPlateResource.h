// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectPtr.h"
#include "UObject/SoftObjectPtr.h"
#include "MediaPlateResource.generated.h"

class UMediaPlaylist;
class UMediaSource;

UENUM()
enum class EMediaPlateResourceType : uint8
{
	Playlist,
	External,
	Asset
};

/**
 * Helper struct to wrap source selection functionality,
 * and enabling the usage of media source properties for places like Remote Control.
 *
 * This struct allows to choose between Asset, External File, Playlist options.
 * It's mainly conceived to be used by MediaPlateComponent.
 *
 * See FMediaPlayerResourceCustomization class for its customization.
 */
USTRUCT(BlueprintType)
struct FMediaPlateResource
{
	GENERATED_BODY()

public:
	/**
	 * Returns the current source playlist, if any
	 */
	MEDIAPLATE_API UMediaPlaylist* GetSourcePlaylist() const;

	/**
	 * Returns the current external media path, if any
	 */
	FStringView GetExternalMediaPath() const { return ExternalMediaPath; }

	/**
	 * Returns the current asset-based Media Source, if any
	 */
	MEDIAPLATE_API UMediaSource* GetMediaAsset() const;

	/**
	 * Return current resource type
	 */
	EMediaPlateResourceType GetResourceType() const { return Type; }

	/**
	 * Set current resource type
	 */
	MEDIAPLATE_API void SetResourceType(EMediaPlateResourceType InType);

	/**
	 * Select asset based media source. Will also update source type to Asset
	 */
	MEDIAPLATE_API void SelectAsset(const UMediaSource* InMediaSource);

	/**
	 * Loads the external source at the specified path, creating a media source with the specified outer.
	 * Will also update source type to External
	 */
	MEDIAPLATE_API void LoadExternalMedia(const FString& InFilePath);

	/**
	 * Select the specified playlist. Will also update source type to Playlist
	 */
	MEDIAPLATE_API void SelectPlaylist(const UMediaPlaylist* InPlaylist);

	UE_DEPRECATED(5.6, "Use UMediaPlateComponent::GetSelectedMediaSource() instead.")
	UMediaSource* GetSelectedMedia() const { return nullptr; }

	UE_DEPRECATED(5.6, "Use UMediaPlateComponent::GetMediaPlaylist() instead.")
	UMediaPlaylist* GetActivePlaylist() const { return nullptr; }

	UE_DEPRECATED(5.6, "Use UMediaPlateComponent::SelectMediaSourceAsset() instead.")
	void SelectAsset(const UMediaSource* InMediaSource, UObject* InOuter)
	{
		SelectAsset(InMediaSource);
	}

	UE_DEPRECATED(5.6, "Use UMediaPlateComponent::SelectExternalMedia() instead.")
	void LoadExternalMedia(const FString& InFilePath, UObject* InOuter)
	{
		LoadExternalMedia(InFilePath);
	}

#if WITH_EDITOR
	UE_DEPRECATED(5.6, "Use UMediaPlateComponent::GetMediaPlaylist()->Modify() instead")
	void Modify() const {}
#endif	

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FMediaPlateResource() = default;
	FMediaPlateResource(const FMediaPlateResource& InOther) = default;
	FMediaPlateResource(FMediaPlateResource&& InOther) = default;
	FMediaPlateResource& operator=(const FMediaPlateResource&) = default;
	FMediaPlateResource& operator=(FMediaPlateResource&&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

private:
friend class UMediaPlateComponent;
friend class FMediaPlateResourceCustomization;
	/**
	 * Initialize member properties from another MediaPlayerResource. Empty or null property will not be copied over.
	 */
	void Init(const FMediaPlateResource& InOther);

	/** Media Source Type */
	UPROPERTY(EditAnywhere, Category = "MediaPlateResource")
	EMediaPlateResourceType Type = EMediaPlateResourceType::Asset;

	/** A path pointing to an external media resource */
	UPROPERTY(EditAnywhere, Category = "MediaPlateResource")
	FString ExternalMediaPath;

	/** Media source coming from MediaSource asset*/
	UPROPERTY(EditAnywhere, Category = "MediaPlateResource")
	TSoftObjectPtr<UMediaSource> MediaAsset;

	/** User facing Playlist asset */
	UPROPERTY(EditAnywhere, Category = "MediaPlateResource")
	TSoftObjectPtr<UMediaPlaylist> SourcePlaylist;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.6, "Moved to UMediaPlateComponent::ExternalMediaSource")
	UPROPERTY(Instanced, meta=(DeprecatedProperty, DeprecationMessage="Moved to UMediaPlateComponent::ExternalMediaSource"))
	TObjectPtr<UMediaSource> ExternalMedia_DEPRECATED;
#endif
};
