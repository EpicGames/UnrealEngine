// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MediaPlateEditor : ModuleRules
{
	public MediaPlateEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
			);
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AssetDefinition",
				"ApplicationCore",
				"CinematicCamera",
				"CoreUObject",
				"DesktopWidgets",
				"EditorStyle",
				"Engine",
				"GeometryCore",
				"InputCore",
				"MediaAssets",
				"MediaCompositing",
				"MediaCompositingEditor",
				"MediaPlate",
				"MediaPlayerEditor",
				"ModelingComponentsEditorOnly",
				"ModelingComponents",
				"MovieScene",
				"PlacementMode",
				"Projects",
				"PropertyEditor",
				"SceneOutliner",
				"Sequencer",
				"Slate",
				"SlateCore",
				"UnrealEd"
			}
		);
	}
}
