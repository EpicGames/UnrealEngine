// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorFactoryMediaPlate.h"

#include "Async/Async.h"
#include "MediaSource.h"
#include "MediaPlate.h"
#include "MediaPlateComponent.h"
#include "MediaPlateEditorModule.h"
#include "MediaPlaylist.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorFactoryMediaPlate)

#define LOCTEXT_NAMESPACE "ActorFactoryMediaPlate"

UActorFactoryMediaPlate::UActorFactoryMediaPlate(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("MediaPlateDisplayName", "Media Plate");
	NewActorClass = AMediaPlate::StaticClass();
}

bool UActorFactoryMediaPlate::CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg)
{
	if (AssetData.IsValid())
	{
		UClass* AssetClass = AssetData.GetClass();
		if ((AssetClass != nullptr) && (AssetClass->IsChildOf(UMediaSource::StaticClass())))
		{
			return true;
		}
		else
		{
			return false;
		}
	}
	else
	{
		return true;
	}
}

void UActorFactoryMediaPlate::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	Super::PostSpawnActor(Asset, NewActor);

	SetUpActor(Asset, NewActor);
}

void UActorFactoryMediaPlate::SetUpActor(UObject* Asset, AActor* Actor)
{
	if (Actor != nullptr)
	{
		AMediaPlate* MediaPlate = CastChecked<AMediaPlate>(Actor);

		// Hook up media source.
		UMediaSource* MediaSource = Cast<UMediaSource>(Asset);
		if ((MediaSource != nullptr) && (MediaPlate->MediaPlateComponent != nullptr))
		{
			// Is this media source from a drag and drop?
			FMediaPlateEditorModule* EditorModule = FModuleManager::LoadModulePtr<FMediaPlateEditorModule>("MediaPlateEditor");
			if (EditorModule != nullptr)
			{
				bool bIsInDragDropCache = EditorModule->RemoveMediaSourceFromDragDropCache(MediaSource);
				if (bIsInDragDropCache && MediaSource->GetOuter() == GetTransientPackage())
				{
					// Yes. Move this out of transient.
					// Can't do it here as the asset is still being used.
					TWeakObjectPtr<UMediaPlateComponent> MediaPlateComponentPtr(MediaPlate->MediaPlateComponent);
					AsyncTask(ENamedThreads::GameThread, [MediaPlateComponentPtr]()
					{
						UMediaPlateComponent* MediaPlateComponent = MediaPlateComponentPtr.Get();
						if (MediaPlateComponent != nullptr)
						{
							if (UMediaSource* MediaSource = MediaPlateComponent->MediaPlateResource.GetMediaAsset())
							{
								MediaSource->Rename(nullptr, MediaPlateComponent);

								// Let's initialize the plate source with the now non-transient MediaSource
								MediaPlateComponent->SelectMediaSourceAsset(MediaSource);
							}
						}
					});
				}
				else
				{
					// MediaSource is non-transient, we can initialize the plate source right away
					MediaPlate->MediaPlateComponent->SelectMediaSourceAsset(MediaSource);
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
