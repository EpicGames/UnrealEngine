// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ActorFactories/ActorFactory.h"
#include "ActorFactoryMediaPlate.generated.h"

UCLASS()
class UActorFactoryMediaPlate : public UActorFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UActorFactory Interface
	virtual bool CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg) override;
	virtual void PostSpawnActor(UObject* Asset, AActor* NewActor) override;
	//~ End UActorFactory Interface

private:
	/**
	 * Set up a media plate actor.
	 * 
	 * @param Asset		Media source asset.
	 * @param Actor		Media plate actor to set up.
	 */
	void SetUpActor(UObject* Asset, AActor* Actor);
};
