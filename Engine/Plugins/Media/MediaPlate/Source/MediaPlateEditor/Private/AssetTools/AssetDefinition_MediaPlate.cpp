// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_MediaPlate.h"
#include "MediaPlateComponent.h"
#include "MediaPlateEditorStyle.h"
#include "Toolkits/MediaPlateEditorToolkit.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_MediaPlate)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

EAssetCommandResult UAssetDefinition_MediaPlate::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	for (UMediaPlateComponent* MediaPlate : OpenArgs.LoadObjects<UMediaPlateComponent>())
	{
		TSharedRef<FMediaPlateEditorToolkit> EditorToolkit = MakeShareable(new FMediaPlateEditorToolkit(FMediaPlateEditorStyle::Get()));
		EditorToolkit->Initialize(MediaPlate, OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost);
	}

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
