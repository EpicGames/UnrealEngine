// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaPlateComponent.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_MediaPlate.generated.h"

UCLASS()
class UAssetDefinition_MediaPlate : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_MediaPlate", "Media Plate"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor::Red; }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UMediaPlateComponent::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { EAssetCategoryPaths::Media };
		return Categories;
	}
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
};
