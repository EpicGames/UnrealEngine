// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaPlateCustomization.h"

#include "CineCameraSettings.h"
#include "Components/StaticMeshComponent.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IMediaAssetsModule.h"
#include "LevelEditorViewport.h"
#include "MediaPlate.h"
#include "MediaPlateComponent.h"
#include "MediaPlateEditorModule.h"
#include "MediaPlateEditorStyle.h"
#include "MediaPlayer.h"
#include "MediaPlayerEditorModule.h"
#include "PropertyCustomizationHelpers.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "Widgets/SMediaPlateEditorMediaDetails.h"

#define LOCTEXT_NAMESPACE "FMediaPlateCustomization"

namespace UE::MediaPlateCustomization::Private
{
	/** Special aspect ratio value for the "custom" preset entry. */
	constexpr float CustomAspectRatioPresetValue = -1.0f;

	/** Special aspect ratio value for the "disable" preset entry. */
	constexpr float DisableAspectRatioPresetValue = 0.0f;
	
	struct FAspectRatioPreset
	{
		FText DisplayName;
		FText Description;
		float AspectRatio = 0;
	};
	
	/**
	 * Hardcoding the aspect ratio presets for Media Plate.
	 * We ensure that all entries are unique to allow reversible mapping from the AR value to the combo box. 
	 */
	static const TArray<FAspectRatioPreset> AspectRatioPresets =
	{
		// Displays and video ARs
		{
			LOCTEXT("ARPresetLabel_16:9", "Widescreen (16:9)"),
			LOCTEXT("ARPresetDesc_16:9", "The standard for high-definition (HD) television, 4K displays, computer monitors, and modern widescreen video."),
			16.0f/9.0f
		},
		{
			LOCTEXT("ARPresetLabel_16:10", "Widescreen+ (16:10)"),
			LOCTEXT("ARPresetDesc_16:10", "Used for many widescreen computer displays and some tablets, offering a slightly taller image than 16:9."),
			16.0f/10.0f
		},
		{
			LOCTEXT("ARPresetLabel_4:3", "Fullscreen (4:3)"),
			LOCTEXT("ARPresetDesc_4:3", "A traditional, older aspect ratio for standard definition TVs (like NTSC and PAL) and some non-widescreen computer monitors."), 
			4.0f/3.0f
		},
		{
			LOCTEXT("ARPresetLabel_21:9", "Ultra-Widescreen (21:9)"),
			LOCTEXT("ARPresetDesc_21:9", "An ultra-widescreen format used for some computer displays and a cinematic feel."),
			21.0f/9.0f
		},
		{
			LOCTEXT("ARPresetLabel_9:16", "Portrait (9:16)"),
			LOCTEXT("ARPresetDesc_9:16", "A vertical aspect ratio (a 90-degree flip of 16:9) popular for video stories on social media platforms like Instagram and Snapchat."),
			9.0f/16.0f
		},
		// Photography and Cinematography ARs
		{
			LOCTEXT("ARPresetLabel_3:2", "35mm film (3:2)"),
			LOCTEXT("ARPresetDesc_3:2", "A traditional aspect ratio that originated with 35mm film, still common in still photography and for print sizes."),
			3.0f/2.0f
		},
		{
			LOCTEXT("ARPresetLabel_1:1", "Square (1:1)"),
			LOCTEXT("ARPresetDesc_1:1", "A perfect square format, often used for social media images and some large format photography."),
			1.0f
		},
		{
			LOCTEXT("ARPresetLabel_5:4", "\"Wide\" (5:4)"),
			LOCTEXT("ARPresetDesc_5:4", "Used in some photography and art prints, especially with large and medium format cameras."),
			5.0f/4.0f
		},
		{
			LOCTEXT("ARPresetLabel_1.85:1", "Flat (1.85:1)"),
			LOCTEXT("ARPresetDesc_1.85:1", "A common aspect ratio for cinema, often referred to as \"flat\"."),
			1.85f/1.0f
		},
		{
			LOCTEXT("ARPresetLabel_2.39:1", "Scope (2.39:1)" ),
			LOCTEXT("ARPresetDesc_2.39:1", "Another cinematic aspect ratio, also known as the \"scope\" format."),
			2.39f/1.0f
		},
	};

	/** Special preset entry for custom entries. */
	static const FAspectRatioPreset CustomAspectRatioPreset =
	{
		LOCTEXT("ARPresetLabel_Custom", "Custom" ),
		LOCTEXT("ARPresetDesc_Custom", "Manually entered aspect ratio not part of presets."),
		CustomAspectRatioPresetValue
	};

	/** Special preset entry to disable the feature (used for letter box ar). */
	static const FAspectRatioPreset DisabledAspectRatioPreset =
	{
		LOCTEXT("Disable", "Disable" ),	// Note: Keeping same name to not change localisation.
		LOCTEXT("Disable_Desc", "Disable the feature."),
		DisableAspectRatioPresetValue
	};
	
	/**
	 * Finds the corresponding preset entry for the given aspect ratio value.
	 * @param InAspectRatio Aspect ratio to search for.
	 * @param bInIncludeDisable Indicate if "disable" entry should be returned.
	 * @return Found preset entry or "custom" entry if not found.
	 */
	const FAspectRatioPreset& FindAspectRatioPreset(float InAspectRatio, bool bInIncludeDisable)
	{
		for (const FAspectRatioPreset& Preset : AspectRatioPresets)
		{
			if (Preset.AspectRatio == InAspectRatio)
			{
				return Preset;
			}
		}

		// Check if this is the "disable" entry.
		if (bInIncludeDisable && InAspectRatio == DisableAspectRatioPresetValue)
		{
			return DisabledAspectRatioPreset;
		}

		// If not found, return the custom preset entry.
		return CustomAspectRatioPreset;
	}

	const FText& FindAspectRatioDisplayName(float InAspectRatio, bool bInIncludeDisable)
	{
		return FindAspectRatioPreset(InAspectRatio, bInIncludeDisable).DisplayName;
	}
}

/* IDetailCustomization interface
 *****************************************************************************/

FMediaPlateCustomization::FMediaPlateCustomization()
{
}

FMediaPlateCustomization::~FMediaPlateCustomization()
{
}

void FMediaPlateCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TWeakPtr<FMediaPlateCustomization> WeakSelf = StaticCastWeakPtr<FMediaPlateCustomization>(AsWeak());

	// Is this the media plate editor window?
	bool bIsMediaPlateWindow = false;

	if (TSharedPtr<const IDetailsView> DetailsView = DetailBuilder.GetDetailsViewSharedPtr())
	{
		TSharedPtr<FTabManager> HostTabManager = DetailsView->GetHostTabManager();
		bIsMediaPlateWindow = (HostTabManager.IsValid() == false);
	}

	// Get style.
	const ISlateStyle* Style = &FMediaPlateEditorStyle::Get().Get();

	CustomizeCategories(DetailBuilder);

	IDetailCategoryBuilder& ControlCategory = DetailBuilder.EditCategory("Control");
	IDetailCategoryBuilder& PlaylistCategory = DetailBuilder.EditCategory("Playlist");
	IDetailCategoryBuilder& GeometryCategory = DetailBuilder.EditCategory("Geometry");
	IDetailCategoryBuilder& MediaDetailsCategory = DetailBuilder.EditCategory("MediaDetails");

	// Get objects we are editing.
	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailBuilder.GetObjectsBeingCustomized(Objects);
	MediaPlatesList.Reserve(Objects.Num());
	for (TWeakObjectPtr<UObject>& Obj : Objects)
	{
		TWeakObjectPtr<UMediaPlateComponent> MediaPlate = Cast<UMediaPlateComponent>(Obj.Get());
		if (MediaPlate.IsValid())
		{
			MediaPlatesList.Add(MediaPlate);
			MeshMode = MediaPlate->GetVisibleMipsTilesCalculations();
		}
	}

	// Add mesh customization.
	AddMeshCustomization(GeometryCategory);

	// Add media plate source 
	MediaPlateResourcePropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMediaPlateComponent, MediaPlateResource));
	if (MediaPlateResourcePropertyHandle)
	{
		PlaylistCategory.AddProperty(MediaPlateResourcePropertyHandle);
	}

	// Add media player playback slider
	if (IMediaPlayerEditorModule* MediaPlayerEditorModule = FModuleManager::LoadModulePtr<IMediaPlayerEditorModule>("MediaPlayerEditor"))
	{
		const TSharedRef<IMediaPlayerSlider> MediaPlayerSlider =
			MediaPlayerEditorModule->CreateMediaPlayerSliderWidget(GetMediaPlayers());

		MediaPlayerSlider->SetSliderHandleColor(FSlateColor(EStyleColor::AccentBlue));
		MediaPlayerSlider->SetVisibleWhenInactive(EVisibility::Visible);

		ControlCategory.AddCustomRow(LOCTEXT("MediaPlatePlaybackPosition", "Playback Position"))
		[
			MediaPlayerSlider
		];
	}

	// Add media control buttons.
	ControlCategory.AddCustomRow(LOCTEXT("MediaPlateControls", "MediaPlate Controls"))
		[
			SNew(SHorizontalBox)
			// Rewind button.
			+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SButton)
						.VAlign(VAlign_Center)
						.IsEnabled_Lambda([WeakSelf]
						{
							if (const TSharedPtr<FMediaPlateCustomization> Self = WeakSelf.Pin())
							{
								return Self->IsButtonEventAllowedForAnyPlate(EMediaPlateEventState::Rewind);
							}
							return false;
						})
						.OnClicked_Lambda([WeakSelf]() -> FReply
						{
							if (const TSharedPtr<FMediaPlateCustomization> Self = WeakSelf.Pin())
							{
								Self->OnButtonEvent(EMediaPlateEventState::Rewind);
								return FReply::Handled();
							}
							return FReply::Unhandled();
						})
						[
							SNew(SImage)
								.ColorAndOpacity(FSlateColor::UseForeground())
								.Image(Style->GetBrush("MediaPlateEditor.RewindMedia.Small"))
								.ToolTipText(LOCTEXT("Rewind", "Rewind the media to the beginning"))
						]
				]

			// Reverse button.
			+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SButton)
						.VAlign(VAlign_Center)
						.IsEnabled_Lambda([WeakSelf]
						{
							if (const TSharedPtr<FMediaPlateCustomization> Self = WeakSelf.Pin())
							{
								return Self->IsButtonEventAllowedForAnyPlate(EMediaPlateEventState::Reverse);
							}
							return false;
						})
						.OnClicked_Lambda([WeakSelf]() -> FReply
						{
							if (const TSharedPtr<FMediaPlateCustomization> Self = WeakSelf.Pin())
							{
								Self->OnButtonEvent(EMediaPlateEventState::Reverse);
								return FReply::Handled();
							}
							return FReply::Unhandled();
						})
						[
							SNew(SImage)
								.ColorAndOpacity(FSlateColor::UseForeground())
								.Image(Style->GetBrush("MediaPlateEditor.ReverseMedia.Small"))
								.ToolTipText(LOCTEXT("Reverse", "Reverse media playback"))
						]
				]

			// Play button.
			+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SButton)
						.VAlign(VAlign_Center)
						.IsEnabled_Lambda([WeakSelf]
						{
							if (const TSharedPtr<FMediaPlateCustomization> Self = WeakSelf.Pin())
							{
								return Self->IsButtonEventAllowedForAnyPlate(EMediaPlateEventState::Play);
							}

							return false;
						})
						.Visibility_Lambda([WeakSelf]
						{
							if (const TSharedPtr<FMediaPlateCustomization> Self = WeakSelf.Pin())
							{
								const bool bAllPlaying = Self->IsTrueForAllPlayers([](const UMediaPlayer* InMediaPlayer)
								{
									return InMediaPlayer->IsPlaying();
								});
								return bAllPlaying ? EVisibility::Collapsed : EVisibility::Visible;
							}

							return EVisibility::Visible;
						})
						.OnClicked_Lambda([WeakSelf]() -> FReply
						{
							if (const TSharedPtr<FMediaPlateCustomization> Self = WeakSelf.Pin())
							{
								Self->OnButtonEvent(EMediaPlateEventState::Play);
								return FReply::Handled();
							}

							return FReply::Unhandled();
						})
						[
							SNew(SImage)
								.ColorAndOpacity(FSlateColor::UseForeground())
								.Image(Style->GetBrush("MediaPlateEditor.PlayMedia.Small"))
								.ToolTipText(LOCTEXT("Play", "Start media playback"))
						]
				]

			// Pause button.
			+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SButton)
						.VAlign(VAlign_Center)
						.IsEnabled_Lambda([WeakSelf]
						{
							if (const TSharedPtr<FMediaPlateCustomization> Self = WeakSelf.Pin())
							{
								return Self->IsButtonEventAllowedForAnyPlate(EMediaPlateEventState::Pause);
							}
							return false;
						})
						.Visibility_Lambda([WeakSelf]
						{
							if (const TSharedPtr<FMediaPlateCustomization> Self = WeakSelf.Pin())
							{
								// We want this logic to be as mutually exclusive with the play button visibility as possible so
								// they don't show at the same time and cause other buttons to move around.
								const bool bAllPaused = Self->IsTrueForAllPlayers([](const UMediaPlayer* InMediaPlayer)
								{
									return !InMediaPlayer->IsPlaying(); // Not using IsPaused() as it is not the logical inverse of IsPlaying.
								});
								return bAllPaused ? EVisibility::Collapsed : EVisibility::Visible;
							}
							return EVisibility::Collapsed;
						})
						.OnClicked_Lambda([WeakSelf]() -> FReply
						{
							if (const TSharedPtr<FMediaPlateCustomization> Self = WeakSelf.Pin())
							{
								Self->OnButtonEvent(EMediaPlateEventState::Pause);
								return FReply::Handled();
							}
							return FReply::Unhandled();
						})
						[
							SNew(SImage)
								.ColorAndOpacity(FSlateColor::UseForeground())
								.Image(Style->GetBrush("MediaPlateEditor.PauseMedia.Small"))
								.ToolTipText(LOCTEXT("Pause", "Pause media playback"))
						]
				]

			// Forward button.
			+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SButton)
					.VAlign(VAlign_Center)
					.IsEnabled_Lambda([WeakSelf]
					{
						if (const TSharedPtr<FMediaPlateCustomization> Self = WeakSelf.Pin())
						{
							return Self->IsButtonEventAllowedForAnyPlate(EMediaPlateEventState::Forward);
						}
						return false;
					})
					.OnClicked_Lambda([WeakSelf]() -> FReply
					{
						if (const TSharedPtr<FMediaPlateCustomization> Self = WeakSelf.Pin())
						{
							Self->OnButtonEvent(EMediaPlateEventState::Forward);
							return FReply::Handled();
						}
						return FReply::Unhandled();
					})
					[
						SNew(SImage)
							.ColorAndOpacity(FSlateColor::UseForeground())
							.Image(Style->GetBrush("MediaPlateEditor.ForwardMedia.Small"))
							.ToolTipText(LOCTEXT("Forward", "Fast forward media playback"))
					]
				]

			// Open button.
			+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SButton)
						.VAlign(VAlign_Center)
						.IsEnabled_Lambda([WeakSelf]
						{
							if (const TSharedPtr<FMediaPlateCustomization> Self = WeakSelf.Pin())
							{
								return Self->IsButtonEventAllowedForAnyPlate(EMediaPlateEventState::Open);
							}
							return true;
						})
						.OnClicked_Lambda([WeakSelf]() -> FReply
						{
							if (const TSharedPtr<FMediaPlateCustomization> Self = WeakSelf.Pin())
							{
								Self->OnButtonEvent(EMediaPlateEventState::Open);
								return FReply::Handled();
							}
							return FReply::Unhandled();
						})
						[
							SNew(SImage)
								.ColorAndOpacity(FSlateColor::UseForeground())
								.Image(Style->GetBrush("MediaPlateEditor.OpenMedia.Small"))
								.ToolTipText(LOCTEXT("Open", "Open the current media"))
						]
				]

			// Close button.
			+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SButton)
						.VAlign(VAlign_Center)
						.IsEnabled_Lambda([WeakSelf]
						{
							if (const TSharedPtr<FMediaPlateCustomization> Self = WeakSelf.Pin())
							{
								return Self->IsButtonEventAllowedForAnyPlate(EMediaPlateEventState::Close);
							}
							return false;
						})
						.OnClicked_Lambda([WeakSelf]() -> FReply
						{
							if (const TSharedPtr<FMediaPlateCustomization> Self = WeakSelf.Pin())
							{
								Self->OnButtonEvent(EMediaPlateEventState::Close);
								return FReply::Handled();
							}
							return FReply::Unhandled();
						})
						[
							SNew(SImage)
								.ColorAndOpacity(FSlateColor::UseForeground())
								.Image(Style->GetBrush("MediaPlateEditor.CloseMedia.Small"))
								.ToolTipText(LOCTEXT("Close", "Close the currently opened media"))
						]
				]
		];


	// Add button to open the media plate editor.
	if (bIsMediaPlateWindow == false)
	{
		PlaylistCategory.AddCustomRow(LOCTEXT("OpenMediaPlate", "Open Media Plate"))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
					.FillWidth(1.f)
					.Padding(0, 5, 10, 5)
					[
						SNew(SButton)
							.ContentPadding(3.0f)
							.VAlign(VAlign_Center)
							.HAlign(HAlign_Center)
							.OnClicked(this, &FMediaPlateCustomization::OnOpenMediaPlate)
							.Text(LOCTEXT("OpenMediaPlate", "Open Media Plate"))
					]
			];

		// Get the first media plate.
		UMediaPlateComponent* FirstMediaPlate = nullptr;
		for (TWeakObjectPtr<UMediaPlateComponent>& MediaPlatePtr : MediaPlatesList)
		{
			UMediaPlateComponent* MediaPlate = MediaPlatePtr.Get();
			if (MediaPlate != nullptr)
			{
				FirstMediaPlate = MediaPlate;
				break;
			}
		}

		if (FirstMediaPlate != nullptr)
		{
			MediaDetailsCategory.AddCustomRow(FText::FromString(TEXT("Media Details")))
			[
				SNew(SMediaPlateEditorMediaDetails, *FirstMediaPlate)
			];
		}
	}
}

void FMediaPlateCustomization::AddMeshCustomization(IDetailCategoryBuilder& InParentCategory)
{
	TWeakPtr<FMediaPlateCustomization> WeakSelf = StaticCastWeakPtr<FMediaPlateCustomization>(AsWeak());

	// Add radio buttons for mesh type.
	InParentCategory.AddCustomRow(FText::FromString("Mesh Selection"))
	[
		SNew(SSegmentedControl<EMediaTextureVisibleMipsTiles>)
			.Value_Lambda([WeakSelf]()
			{
				const TSharedPtr<FMediaPlateCustomization> Self = WeakSelf.Pin();
				return Self.IsValid() ? Self->MeshMode : EMediaTextureVisibleMipsTiles::None;
			})
			.OnValueChanged(this, &FMediaPlateCustomization::SetMeshMode)

		+ SSegmentedControl<EMediaTextureVisibleMipsTiles>::Slot(EMediaTextureVisibleMipsTiles::Plane)
			.Text(LOCTEXT("Plane", "Plane"))
			.ToolTip(LOCTEXT("Plane_ToolTip",
				"Select this if you want to use a standard plane for the mesh."))

		+ SSegmentedControl<EMediaTextureVisibleMipsTiles>::Slot(EMediaTextureVisibleMipsTiles::Sphere)
			.Text(LOCTEXT("Sphere", "Sphere"))
			.ToolTip(LOCTEXT("Sphere_ToolTip",
				"Select this if you want to use a spherical object for the mesh."))

		+ SSegmentedControl<EMediaTextureVisibleMipsTiles>::Slot(EMediaTextureVisibleMipsTiles::None)
			.Text(LOCTEXT("Custom", "Custom"))
			.ToolTip(LOCTEXT("Custom_ToolTip",
				"Select this if you want to provide your own mesh."))
	];

	// Visibility attributes.
	TAttribute<EVisibility> MeshCustomVisibility(this, &FMediaPlateCustomization::ShouldShowMeshCustomWidgets);
	TAttribute<EVisibility> MeshPlaneVisibility(this, &FMediaPlateCustomization::ShouldShowMeshPlaneWidgets);
	TAttribute<EVisibility> MeshSphereVisibility(this, &FMediaPlateCustomization::ShouldShowMeshSphereWidgets);

	// Add aspect ratio.
	InParentCategory.AddCustomRow(FText::FromString("Mesh Selection"))
		.Visibility(MeshPlaneVisibility)
		.NameContent()
		[
			SNew(STextBlock)
				.Text(LOCTEXT("AspectRatio", "Aspect Ratio"))
				.ToolTipText(LOCTEXT("AspectRatio_ToolTip",
				"Sets the aspect ratio of the plane showing the media.\nChanging this will change the scale of the mesh component."))
				.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)

			// Presets button.
			+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SComboButton)
						.OnGetMenuContent(this, &FMediaPlateCustomization::OnGetAspectRatios)
						.ContentPadding(2.0f)
						.ButtonContent()
						[
							SNew(STextBlock)
								.ToolTipText(LOCTEXT("Presets_ToolTip", "Select one of the presets for the aspect ratio."))
								.Text(this, &FMediaPlateCustomization::OnGetAspectRatioText)
						]
				]

			// Numeric entry box.
			+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SSpinBox<float>)
						.Value(this, &FMediaPlateCustomization::GetAspectRatio)
						.MinValue(0.0f)
						.OnValueChanged(this, &FMediaPlateCustomization::SetAspectRatio)
				]
		];

		// Add letterbox aspect ratio.
		InParentCategory.AddCustomRow(FText::FromString("Aspect Ratio"))
			.Visibility(MeshPlaneVisibility)
			.NameContent()
			[
				SNew(STextBlock)
					.Text(LOCTEXT("LetterboxAspectRatio", "Letterbox Aspect Ratio"))
					.ToolTipText(LOCTEXT("LetterboxAspectRatio_ToolTip",
						"Sets the aspect ratio of the whole screen.\n"
						"If the screen is larger than the media then letterboxes will be added."))
					.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			[
				SNew(SHorizontalBox)

				// Presets button.
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SComboButton)
						.OnGetMenuContent(this, &FMediaPlateCustomization::OnGetLetterboxAspectRatios)
						.ContentPadding(2.0f)
						.ButtonContent()
						[
							SNew(STextBlock)
								.ToolTipText(LOCTEXT("Presets_ToolTip", "Select one of the presets for the aspect ratio."))
								.Text(this, &FMediaPlateCustomization::OnGetLetterboxAspectRatioText)
						]
			]

		// Numeric entry box.
		+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SSpinBox<float>)
					.Value(this, &FMediaPlateCustomization::GetLetterboxAspectRatio)
					.MinValue(0.0f)
					.OnValueChanged(this, &FMediaPlateCustomization::SetLetterboxAspectRatio)
			]
		];

	// Add auto aspect ratio.
	InParentCategory.AddCustomRow(FText::FromString("Aspect Ratio"))
		.Visibility(MeshPlaneVisibility)
		.NameContent()
		[
			SNew(STextBlock)
				.Text(LOCTEXT("AutoAspectRatio", "Auto Aspect Ratio"))
				.ToolTipText(LOCTEXT("AutoAspectRatio_ToolTip",
					"Sets the aspect ratio to match the media."))
				.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SCheckBox)
				.IsChecked(this, &FMediaPlateCustomization::IsAspectRatioAuto)
				.OnCheckStateChanged(this, &FMediaPlateCustomization::SetIsAspectRatioAuto)
		];

	// Add sphere horizontal arc.
	InParentCategory.AddCustomRow(FText::FromString("Horizontal Arc"))
		.Visibility(MeshSphereVisibility)
		.NameContent()
		[
			SNew(STextBlock)
				.Text(LOCTEXT("HorizontalArc", "Horizontal Arc"))
				.ToolTipText(LOCTEXT("HorizontalArc_ToolTip",
				"Sets the horizontal arc size of the sphere in degrees.\nFor example 360 for a full circle, 180 for a half circle."))
				.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SNumericEntryBox<float>)
				.Value(this, &FMediaPlateCustomization::GetMeshHorizontalRange)
				.OnValueChanged(this, &FMediaPlateCustomization::SetMeshHorizontalRange)
		];

	// Add sphere vertical arc.
	InParentCategory.AddCustomRow(FText::FromString("Vertical Arc"))
		.Visibility(MeshSphereVisibility)
		.NameContent()
		[
			SNew(STextBlock)
				.Text(LOCTEXT("VerticalArc", "Vertical Arc"))
				.ToolTipText(LOCTEXT("VerticalArc_ToolTip",
				"Sets the vertical arc size of the sphere in degrees.\nFor example 180 for a half circle, 90 for a quarter circle."))
				.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SNumericEntryBox<float>)
				.Value(this, &FMediaPlateCustomization::GetMeshVerticalRange)
				.OnValueChanged(this, &FMediaPlateCustomization::SetMeshVerticalRange)
		];

	// Add static mesh.
	InParentCategory.AddCustomRow(FText::FromString("Static Mesh"))
		.Visibility(MeshCustomVisibility)
		.NameContent()
		[
			SNew(STextBlock)
				.Text(LOCTEXT("StaticMesh", "Static Mesh"))
				.ToolTipText(LOCTEXT("StaticMesh_Tooltip", "The static mesh to use."))
				.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SObjectPropertyEntryBox)
				.AllowedClass(UStaticMesh::StaticClass())
				.ThumbnailPool(UThumbnailManager::Get().GetSharedThumbnailPool())
				.ObjectPath(this, &FMediaPlateCustomization::GetStaticMeshPath)
				.OnObjectChanged(this, &FMediaPlateCustomization::OnStaticMeshChanged)
		];
}

EVisibility FMediaPlateCustomization::ShouldShowMeshCustomWidgets() const
{
	return (MeshMode == EMediaTextureVisibleMipsTiles::None) ? EVisibility::Visible : EVisibility::Hidden;
}

EVisibility FMediaPlateCustomization::ShouldShowMeshPlaneWidgets() const
{
	return (MeshMode == EMediaTextureVisibleMipsTiles::Plane) ? EVisibility::Visible : EVisibility::Hidden;
}

EVisibility FMediaPlateCustomization::ShouldShowMeshSphereWidgets() const
{
	return (MeshMode == EMediaTextureVisibleMipsTiles::Sphere) ? EVisibility::Visible : EVisibility::Hidden;
}

void FMediaPlateCustomization::SetMeshMode(EMediaTextureVisibleMipsTiles InMode)
{
	if (MeshMode != InMode)
	{
		const FScopedTransaction Transaction(LOCTEXT("SetMeshMode", "Media Plate Mesh Changed"));

		MeshMode = InMode;
		for (const TWeakObjectPtr<UMediaPlateComponent>& MediaPlatePtr : MediaPlatesList)
		{
			UMediaPlateComponent* MediaPlate = MediaPlatePtr.Get();
			if (MediaPlate != nullptr)
			{
				MediaPlate->Modify();

				// Update the setting in the media plate.
				MediaPlate->SetVisibleMipsTilesCalculations(MeshMode);

				// Set the appropriate mesh.
				if (MeshMode == EMediaTextureVisibleMipsTiles::Plane)
				{
					MeshCustomization.SetPlaneMesh(MediaPlate);
				}
				else
				{
					// Letterboxes are only for planes.
					SetLetterboxAspectRatio(0.0f);

					if (MeshMode == EMediaTextureVisibleMipsTiles::Sphere)
					{
						SetSphereMesh(MediaPlate);
					}
				}
			}
		}
	}
}

void FMediaPlateCustomization::SetSphereMesh(UMediaPlateComponent* MediaPlate)
{
	MeshCustomization.SetSphereMesh(MediaPlate);
}

ECheckBoxState FMediaPlateCustomization::IsAspectRatioAuto() const
{
	ECheckBoxState State = ECheckBoxState::Undetermined;

	for (const TWeakObjectPtr<UMediaPlateComponent>& MediaPlatePtr : MediaPlatesList)
	{
		UMediaPlateComponent* MediaPlate = MediaPlatePtr.Get();
		if (MediaPlate != nullptr)
		{
			ECheckBoxState NewState = MediaPlate->GetIsAspectRatioAuto() ? ECheckBoxState::Checked :
				ECheckBoxState::Unchecked;
			if (State == ECheckBoxState::Undetermined)
			{
				State = NewState;
			}
			else if (State != NewState)
			{
				// If the media plates have different states then return undetermined.
				State = ECheckBoxState::Undetermined;
				break;
			}
		}
	}

	return State;
}

void FMediaPlateCustomization::SetIsAspectRatioAuto(ECheckBoxState State)
{
	const FScopedTransaction Transaction(LOCTEXT("SetIsAspectRatioAuto", "Media Plate Set Is Aspect Ratio Auto"));
	bool bEnable = (State == ECheckBoxState::Checked);

	// Loop through all our objects.
	for (const TWeakObjectPtr<UMediaPlateComponent>& MediaPlatePtr : MediaPlatesList)
	{
		UMediaPlateComponent* MediaPlate = MediaPlatePtr.Get();
		if (MediaPlate != nullptr)
		{
			MediaPlate->Modify();
			MediaPlate->SetIsAspectRatioAuto(bEnable);
		}
	}
}

TSharedRef<SWidget> FMediaPlateCustomization::OnGetAspectRatios()
{
	FMenuBuilder MenuBuilder(true, NULL);
	AddAspectRatiosToMenuBuilder(MenuBuilder, &FMediaPlateCustomization::SetAspectRatio);

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> FMediaPlateCustomization::OnGetLetterboxAspectRatios()
{
	FMenuBuilder MenuBuilder(true, NULL);
	AddAspectRatiosToMenuBuilder(MenuBuilder, &FMediaPlateCustomization::SetLetterboxAspectRatio);

	// Add the extra "disable" entry.
	using namespace UE::MediaPlateCustomization::Private; 
	FUIAction Action = FUIAction(FExecuteAction::CreateSP(this, &FMediaPlateCustomization::SetLetterboxAspectRatio, DisabledAspectRatioPreset.AspectRatio));
	MenuBuilder.AddMenuEntry(DisabledAspectRatioPreset.DisplayName, DisabledAspectRatioPreset.Description, FSlateIcon(), Action);

	return MenuBuilder.MakeWidget();
}

FText FMediaPlateCustomization::OnGetAspectRatioText() const
{
	using namespace UE::MediaPlateCustomization::Private;
	return FindAspectRatioDisplayName(GetAspectRatio(), /*bInIncludeDisable*/false);
}

FText FMediaPlateCustomization::OnGetLetterboxAspectRatioText() const
{
	using namespace UE::MediaPlateCustomization::Private;
	return FindAspectRatioDisplayName(GetLetterboxAspectRatio(), /*bInIncludeDisable*/true);
}

void FMediaPlateCustomization::AddAspectRatiosToMenuBuilder(FMenuBuilder& MenuBuilder,
	void (FMediaPlateCustomization::* Func)(float))
{
	using namespace UE::MediaPlateCustomization::Private;
	for (const FAspectRatioPreset& Preset : AspectRatioPresets)
	{
		FUIAction Action = FUIAction(FExecuteAction::CreateSP(this,
			Func, Preset.AspectRatio));
		MenuBuilder.AddMenuEntry(Preset.DisplayName, Preset.Description, FSlateIcon(), Action);
	}
}

void FMediaPlateCustomization::SetAspectRatio(float AspectRatio)
{
	// Loop through all our objects.
	for (const TWeakObjectPtr<UMediaPlateComponent>& MediaPlatePtr : MediaPlatesList)
	{
		UMediaPlateComponent* MediaPlate = MediaPlatePtr.Get();
		if (MediaPlate != nullptr)
		{
			MediaPlate->SetAspectRatio(AspectRatio);
		}
	}

	// Invalidate the viewport so we can see the mesh change.
	if (GCurrentLevelEditingViewportClient != nullptr)
	{
		GCurrentLevelEditingViewportClient->Invalidate();
	}
}

float FMediaPlateCustomization::GetAspectRatio() const
{
	// Loop through our objects.
	for (const TWeakObjectPtr<UMediaPlateComponent>& MediaPlatePtr : MediaPlatesList)
	{
		UMediaPlateComponent* MediaPlate = MediaPlatePtr.Get();
		if (MediaPlate != nullptr)
		{
			return MediaPlate->GetAspectRatio();
		}
	}

	return 1.0f;
}

void FMediaPlateCustomization::SetLetterboxAspectRatio(float AspectRatio)
{
	for (const TWeakObjectPtr<UMediaPlateComponent>& MediaPlatePtr : MediaPlatesList)
	{
		UMediaPlateComponent* MediaPlate = MediaPlatePtr.Get();
		if (MediaPlate != nullptr)
		{
			MediaPlate->SetLetterboxAspectRatio(AspectRatio);
		}
	}

	// Invalidate the viewport so we can see the mesh change.
	if (GCurrentLevelEditingViewportClient != nullptr)
	{
		GCurrentLevelEditingViewportClient->Invalidate();
	}
}


float FMediaPlateCustomization::GetLetterboxAspectRatio() const
{
	for (const TWeakObjectPtr<UMediaPlateComponent>& MediaPlatePtr : MediaPlatesList)
	{
		UMediaPlateComponent* MediaPlate = MediaPlatePtr.Get();
		if (MediaPlate != nullptr)
		{
			return MediaPlate->GetLetterboxAspectRatio();
		}
	}

	return 0.0f;
}

void FMediaPlateCustomization::SetMeshHorizontalRange(float HorizontalRange)
{
	HorizontalRange = FMath::Clamp(HorizontalRange, 1.0f, 360.0f);
	TOptional VerticalRange = GetMeshVerticalRange();
	if (VerticalRange.IsSet())
	{
		FVector2D MeshRange = FVector2D(HorizontalRange, VerticalRange.GetValue());
		SetMeshRange(MeshRange);
	}
}

TOptional<float> FMediaPlateCustomization::GetMeshHorizontalRange() const
{
	// Loop through our objects.
	for (const TWeakObjectPtr<UMediaPlateComponent>& MediaPlatePtr : MediaPlatesList)
	{
		UMediaPlateComponent* MediaPlate = MediaPlatePtr.Get();
		if (MediaPlate != nullptr)
		{
			return static_cast<float>(MediaPlate->GetMeshRange().X);
		}
	}

	return TOptional<float>();
}

void FMediaPlateCustomization::SetMeshVerticalRange(float VerticalRange)
{
	VerticalRange = FMath::Clamp(VerticalRange, 1.0f, 180.0f);
	TOptional HorizontalRange = GetMeshHorizontalRange();
	if (HorizontalRange.IsSet())
	{
		FVector2D MeshRange = FVector2D(HorizontalRange.GetValue(), VerticalRange);
		SetMeshRange(MeshRange);
	}
}

TOptional<float> FMediaPlateCustomization::GetMeshVerticalRange() const
{
	// Loop through our objects.
	for (const TWeakObjectPtr<UMediaPlateComponent>& MediaPlatePtr : MediaPlatesList)
	{
		UMediaPlateComponent* MediaPlate = MediaPlatePtr.Get();
		if (MediaPlate != nullptr)
		{
			return static_cast<float>(MediaPlate->GetMeshRange().Y);
		}
	}

	return TOptional<float>();
}

void FMediaPlateCustomization::SetMeshRange(FVector2D Range)
{
	const FScopedTransaction Transaction(LOCTEXT("SetMeshRange", "Media Plate Set Mesh Range"));

	// Loop through all our objects.
	for (const TWeakObjectPtr<UMediaPlateComponent>& MediaPlatePtr : MediaPlatesList)
	{
		UMediaPlateComponent* MediaPlate = MediaPlatePtr.Get();
		if (MediaPlate != nullptr)
		{
			if (MediaPlate->GetMeshRange() != Range)
			{
				MediaPlate->Modify();
				MediaPlate->SetMeshRange(Range);
				SetSphereMesh(MediaPlate);
			}
		}
	}
}


FString FMediaPlateCustomization::GetStaticMeshPath() const
{
	FString Path;

	// Get the first media plate.
	if (MediaPlatesList.Num() > 0)
	{
		UMediaPlateComponent* MediaPlate = MediaPlatesList[0].Get();
		if (MediaPlate != nullptr)
		{
			UStaticMeshComponent* StaticMeshComponent = MediaPlate->StaticMeshComponent;
			if (StaticMeshComponent != nullptr)
			{
				UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh();
				if (StaticMesh != nullptr)
				{
					Path = StaticMesh->GetPathName();
				}
			}
		}
	}

	return Path;
}

void FMediaPlateCustomization::OnStaticMeshChanged(const FAssetData& AssetData)
{
	const FScopedTransaction Transaction(LOCTEXT("OnStaticMeshChanged", "Media Plate Custom Mesh Changed"));

	// Update the static mesh.
	UStaticMesh* StaticMesh = Cast<UStaticMesh>(AssetData.GetAsset());
	for (TWeakObjectPtr<UMediaPlateComponent>& MediaPlatePtr : MediaPlatesList)
	{
		UMediaPlateComponent* MediaPlate = MediaPlatePtr.Get();
		if (MediaPlate != nullptr)
		{
			MeshCustomization.SetCustomMesh(MediaPlate, StaticMesh);
		}
	}
}

bool FMediaPlateCustomization::IsButtonEventAllowedForAnyPlate(EMediaPlateEventState InState) const
{
	// Returns true if any of the selected plate allows the action.
	for (const TWeakObjectPtr<UMediaPlateComponent>& MediaPlateWeak : MediaPlatesList)
	{
		if (UMediaPlateComponent* MediaPlate = MediaPlateWeak.Get())
		{
			if (MediaPlate->IsEventStateChangeAllowed(InState)
				&& IsMediaPlateEventAllowedForPlayer(InState, MediaPlate->GetMediaPlayer()))
			{
				return true;
			}
		}
	}
	return false;
}

bool FMediaPlateCustomization::IsMediaPlateEventAllowedForPlayer(EMediaPlateEventState InState, UMediaPlayer* InMediaPlayer)
{
	// Note: centralize the state switch conditions here to make it easier to maintain.
	if (!InMediaPlayer)
	{
		return false;
	}
	
	switch (InState)
	{
	case EMediaPlateEventState::Play:
		// Is player paused or fast forwarding/rewinding?
		return InMediaPlayer->IsReady()
			&& (!InMediaPlayer->IsPlaying() || InMediaPlayer->GetRate() != 1.0f);

	case EMediaPlateEventState::PlayReverse:
		// Is player paused or fast forwarding/rewinding?
		return InMediaPlayer->IsReady() && InMediaPlayer->SupportsRate(-1.f, /* Unthinned */ true)
			&& (!InMediaPlayer->IsPlaying() || InMediaPlayer->GetRate() != -1.0f);

	case EMediaPlateEventState::Open:
		return true; // The condition is implemented by the media plate already.

	case EMediaPlateEventState::Close:
		return !InMediaPlayer->GetUrl().IsEmpty();

	case EMediaPlateEventState::Pause:
		return InMediaPlayer->CanPause() && !InMediaPlayer->IsPaused();

	case EMediaPlateEventState::Reverse:
		return InMediaPlayer->IsReady() && InMediaPlayer->SupportsRate(UMediaPlateComponent::GetReverseRate(InMediaPlayer), false);

	case EMediaPlateEventState::Forward:
		return InMediaPlayer->IsReady() && InMediaPlayer->SupportsRate(UMediaPlateComponent::GetForwardRate(InMediaPlayer), false);

	case EMediaPlateEventState::Rewind:
		return InMediaPlayer->IsReady() && InMediaPlayer->SupportsSeeking() && InMediaPlayer->GetTime() > FTimespan::Zero();

	case EMediaPlateEventState::JumpToEnd:
		return InMediaPlayer->IsReady() && InMediaPlayer->SupportsSeeking() && InMediaPlayer->GetTime() < InMediaPlayer->GetDuration();

	case EMediaPlateEventState::StepForward:
	case EMediaPlateEventState::StepBackward:
		return InMediaPlayer->IsReady() && InMediaPlayer->SupportsSeeking() && InMediaPlayer->IsPaused();
	
	//case EMediaPlateEventState::Next:	// was not implemented
	//case EMediaPlateEventState::Previous: // was not implemented

	default:
		return true;
	}
}

void FMediaPlateCustomization::OnButtonEvent(EMediaPlateEventState InState)
{
	HandleMediaPlateEvent(MediaPlatesList, InState);
}

void FMediaPlateCustomization::HandleMediaPlateEvent(TConstArrayView<TWeakObjectPtr<UMediaPlateComponent>> InMediaPlates, EMediaPlateEventState InState)
{
	TArray<FString> ActorsPathNames;
	ActorsPathNames.Reserve(InMediaPlates.Num());
	for (const TWeakObjectPtr<UMediaPlateComponent>& MediaPlateWeak : InMediaPlates)
	{
		UMediaPlateComponent* MediaPlate = MediaPlateWeak.Get();

		// Note: because of multi-selection and the possibility of different player states, we need check restrictions per plate again.
		if (!MediaPlate || !MediaPlate->IsEventStateChangeAllowed(InState) || !IsMediaPlateEventAllowedForPlayer(InState, MediaPlate->GetMediaPlayer()))
		{
			continue;
		}

		ActorsPathNames.Add(MediaPlate->GetOwner()->GetPathName());

		if (InState == EMediaPlateEventState::Open)
		{
			// Tell the editor module that this media plate is playing.
			if (FMediaPlateEditorModule* EditorModule = FModuleManager::LoadModulePtr<FMediaPlateEditorModule>("MediaPlateEditor"))
			{
				EditorModule->MediaPlateStartedPlayback(MediaPlate);
			}
		}

		MediaPlate->SwitchStates(InState);
	}

	if (IMediaAssetsModule* MediaAssets = FModuleManager::LoadModulePtr<IMediaAssetsModule>("MediaAssets"))
	{
		MediaAssets->BroadcastOnMediaStateChangedEvent(ActorsPathNames, (uint8)InState);
	}
}

FReply FMediaPlateCustomization::OnOpenMediaPlate()
{
	// Get all our objects.
	TArray<UObject*> AssetArray;
	for (TWeakObjectPtr<UMediaPlateComponent>& MediaPlatePtr : MediaPlatesList)
	{
		UMediaPlateComponent* MediaPlate = MediaPlatePtr.Get();
		if (MediaPlate != nullptr)
		{
			AssetArray.Add(MediaPlate);
		}
	}

	// Open the editor.
	if (GEditor && AssetArray.Num() > 0)
	{
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAssets(AssetArray);
	}

	return FReply::Handled();
}

void FMediaPlateCustomization::StopMediaPlates()
{
	OnButtonEvent(EMediaPlateEventState::Close);
}

TArray<TWeakObjectPtr<UMediaPlayer>> FMediaPlateCustomization::GetMediaPlayers() const
{
	TArray<TWeakObjectPtr<UMediaPlayer>> MediaPlayers;
	MediaPlayers.Reserve(MediaPlatesList.Num());
	
	for (const TWeakObjectPtr<UMediaPlateComponent>& MediaPlateWeak : MediaPlatesList)
	{
		if (UMediaPlateComponent* MediaPlate = MediaPlateWeak.Get())
		{
			if (UMediaPlayer* MediaPlayer = MediaPlate->GetMediaPlayer())
			{
				MediaPlayers.Add(MediaPlayer);
			}
		}
	}
	return MediaPlayers;
}

bool FMediaPlateCustomization::IsTrueForAllPlayers(TFunctionRef<bool(const UMediaPlayer*)> InPredicate) const
{
	bool bPredicateCalled = false;

	for (const TWeakObjectPtr<UMediaPlateComponent>& MediaPlateWeak : MediaPlatesList)
	{
		if (UMediaPlateComponent* MediaPlate = MediaPlateWeak.Get())
		{
			if (const UMediaPlayer* MediaPlayer = MediaPlate->GetMediaPlayer())
			{
				if (!InPredicate(MediaPlayer))
				{
					return false;
				}
				bPredicateCalled = true;
			}
		}
	}

	return bPredicateCalled;
}

void FMediaPlateCustomization::CustomizeCategories(IDetailLayoutBuilder& InDetailBuilder)
{
	static const FName PropertyEditor("PropertyEditor");
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(PropertyEditor);

	// Rearrange Categories

	const FName MediaPlateComponentName = UMediaPlateComponent::StaticClass()->GetFName();

	const TSharedRef<FPropertySection> GeneralSection = PropertyModule.FindOrCreateSection(MediaPlateComponentName, TEXT("General"), LOCTEXT("General", "General"));
	GeneralSection->AddCategory(TEXT("Control"));
	GeneralSection->AddCategory(TEXT("Geometry"));
	GeneralSection->AddCategory(TEXT("Playlist"));
	GeneralSection->AddCategory(TEXT("MediaDetails"));
	GeneralSection->AddCategory(TEXT("MediaTexture"));
	GeneralSection->AddCategory(TEXT("Materials"));
	GeneralSection->AddCategory(TEXT("EXR Tiles & Mips"));
	GeneralSection->AddCategory(TEXT("Media Cache"));
	GeneralSection->AddCategory(TEXT("Advanced"));

	const TSharedRef<FPropertySection> MediaSection = PropertyModule.FindOrCreateSection(MediaPlateComponentName, TEXT("Media"), LOCTEXT("Media", "Media"));
	MediaSection->AddCategory(TEXT("Playlist"));
	MediaSection->AddCategory(TEXT("MediaDetails"));
	MediaSection->AddCategory(TEXT("Media Cache"));

	const TSharedRef<FPropertySection> EXRSection = PropertyModule.FindOrCreateSection(MediaPlateComponentName, TEXT("EXR"), LOCTEXT("EXR", "EXR"));
	EXRSection->AddCategory(TEXT("MediaDetails"));
	EXRSection->AddCategory(TEXT("EXR Tiles & Mips"));
	EXRSection->AddCategory(TEXT("Media Cache"));

	const TSharedRef<FPropertySection> RenderingSection = PropertyModule.FindOrCreateSection(MediaPlateComponentName, TEXT("Rendering"), LOCTEXT("Rendering", "Rendering"));
	RenderingSection->AddCategory(TEXT("Geometry"));
	RenderingSection->AddCategory(TEXT("Materials"));
	RenderingSection->AddCategory(TEXT("MediaTexture"));
	RenderingSection->AddCategory(TEXT("Mobility"));
	RenderingSection->AddCategory(TEXT("Transform"));
	RenderingSection->AddCategory(TEXT("TransformCommon"));
	RenderingSection->RemoveCategory(TEXT("Lighting"));
	RenderingSection->AddCategory(TEXT("MediaTexture"));
	RenderingSection->RemoveCategory(TEXT("MaterialParameters"));
	RenderingSection->RemoveCategory(TEXT("Mobile"));
	RenderingSection->RemoveCategory(TEXT("RayTracing"));
	RenderingSection->RemoveCategory(TEXT("Rendering"));
	RenderingSection->RemoveCategory(TEXT("TextureStreaming"));
	RenderingSection->RemoveCategory(TEXT("VirtualTexture"));

	// Hide unwanted Categories

	const FName MediaPlateName = AMediaPlate::StaticClass()->GetFName();

	const TSharedRef<FPropertySection> MediaPlateMiscSection = PropertyModule.FindOrCreateSection(MediaPlateName, TEXT("Misc"), LOCTEXT("Misc", "Misc"));
	MediaPlateMiscSection->RemoveCategory("AssetUserData");
	MediaPlateMiscSection->RemoveCategory("Cooking");
	MediaPlateMiscSection->RemoveCategory("Input");
	MediaPlateMiscSection->RemoveCategory("Navigation");
	MediaPlateMiscSection->RemoveCategory("Replication");
	MediaPlateMiscSection->RemoveCategory("Tags");

	const TSharedRef<FPropertySection> MediaPlateStreamingSection = PropertyModule.FindOrCreateSection(MediaPlateName, TEXT("Streaming"), LOCTEXT("Streaming", "Streaming"));
	MediaPlateStreamingSection->RemoveCategory("Data Layers");
	MediaPlateStreamingSection->RemoveCategory("HLOD");
	MediaPlateStreamingSection->RemoveCategory("World Partition");

	const TSharedRef<FPropertySection> MediaPlateLODSection = PropertyModule.FindOrCreateSection(MediaPlateName, TEXT("LOD"), LOCTEXT("LOD", "LOD"));
	MediaPlateLODSection->RemoveCategory("HLOD");
	MediaPlateLODSection->RemoveCategory("LOD");

	const TSharedRef<FPropertySection> MediaPlatePhysicsSection = PropertyModule.FindOrCreateSection(MediaPlateName, TEXT("Physics"), LOCTEXT("Physics", "Physics"));
	MediaPlatePhysicsSection->RemoveCategory("Collision");
	MediaPlatePhysicsSection->RemoveCategory("Physics");

	// Hide the static mesh.
	IDetailCategoryBuilder& StaticMeshCategory = InDetailBuilder.EditCategory("StaticMesh");
	StaticMeshCategory.SetCategoryVisibility(false);

	IDetailCategoryBuilder& ControlCategory = InDetailBuilder.EditCategory("Control");
	IDetailCategoryBuilder& MediaDetailsCategory = InDetailBuilder.EditCategory("MediaDetails");
	IDetailCategoryBuilder& PlaylistCategory = InDetailBuilder.EditCategory("Playlist");
	IDetailCategoryBuilder& GeometryCategory = InDetailBuilder.EditCategory("Geometry");
	IDetailCategoryBuilder& MediaTextureCategory = InDetailBuilder.EditCategory("MediaTexture");
	IDetailCategoryBuilder& MaterialsCategory = InDetailBuilder.EditCategory("Materials");
	IDetailCategoryBuilder& TilesMipsCategory = InDetailBuilder.EditCategory("EXR Tiles & Mips");
	IDetailCategoryBuilder& MediaCacheCategory = InDetailBuilder.EditCategory("Media Cache");

	// Rename Media Cache category and look ahead property
	MediaCacheCategory.SetDisplayName(FText::FromString(TEXT("Cache")));

	const TSharedRef<IPropertyHandle> CacheSettingsProperty = InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMediaPlateComponent, CacheSettings));
	if (const auto LookAheadTimeProperty = CacheSettingsProperty->GetChildHandle(TEXT("TimeToLookAhead")))
	{
		LookAheadTimeProperty->SetPropertyDisplayName(FText::FromString("Look Ahead Time"));
	}

	// Start from a Priority value which places these categories after the Transform one
	uint32 Priority = 2010;
	ControlCategory.SetSortOrder(Priority++);
	GeometryCategory.SetSortOrder(Priority++);
	PlaylistCategory.SetSortOrder(Priority++);
	MediaDetailsCategory.SetSortOrder(Priority++);
	MediaTextureCategory.SetSortOrder(Priority++);
	MaterialsCategory.SetSortOrder(Priority++);
	TilesMipsCategory.SetSortOrder(Priority++);
	MediaCacheCategory.SetSortOrder(Priority);
}
#undef LOCTEXT_NAMESPACE
