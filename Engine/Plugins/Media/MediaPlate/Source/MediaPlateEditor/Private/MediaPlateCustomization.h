// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Editor.h"
#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "MediaPlateCustomizationMesh.h"
#include "MediaTextureTracker.h"
#include "Styling/SlateTypes.h"

class FMenuBuilder;
class IDetailCategoryBuilder;
class IDetailLayoutBuilder;
class IPropertyHandle;
class SWidget;
class UMediaPlateComponent;
class UMediaPlayer;
enum class EMediaPlateEventState : uint8;

/**
 * Implements a details view customization for the UMediaPlateComponent class.
 */
class FMediaPlateCustomization
	: public IDetailCustomization
{
public:

	FMediaPlateCustomization();
	~FMediaPlateCustomization();

	//~ IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

	/**
	 * Creates an instance of this class.
	 *
	 * @return The new instance.
	 */
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShareable(new FMediaPlateCustomization());
	}

	/**
	* Determines if the event state switch is allowed for the given media player.
	* Those are the UI only conditions to mirror the backend's conditions.
	*/
	static bool IsMediaPlateEventAllowedForPlayer(EMediaPlateEventState InState, UMediaPlayer* InMediaPlayer);

	/**
	* Changes the state of selected media plates and broadcasts the event to the remote endpoints.
	*/
	static void HandleMediaPlateEvent(TConstArrayView<TWeakObjectPtr<UMediaPlateComponent>> InMediaPlates, EMediaPlateEventState InState);

private:
	static void CustomizeCategories(IDetailLayoutBuilder& InDetailBuilder);

	/** Property change delegate used for static mesh material changes. */
	FDelegateHandle PropertyChangeDelegate;
	/** List of the media plates we are editing. */
	TArray<TWeakObjectPtr<UMediaPlateComponent>> MediaPlatesList;

	/** Whether we have a plane, sphere, etc. */
	EMediaTextureVisibleMipsTiles MeshMode = EMediaTextureVisibleMipsTiles::None;

	/** Handles mesh stuff. */
	FMediaPlateCustomizationMesh MeshCustomization;

	/** Property handle of the currently customized Media Plate Resource */
	TSharedPtr<IPropertyHandle> MediaPlateResourcePropertyHandle;

	/**
	 * Adds widgets for editing the mesh.
	 */
	void AddMeshCustomization(IDetailCategoryBuilder& InParentCategory);

	/**
	 * Controls visibility for widgets for custom meshes.
	 */
	EVisibility ShouldShowMeshCustomWidgets() const;

	/**
	 * Controls visibility for widgets for plane meshes.
	 */
	EVisibility ShouldShowMeshPlaneWidgets() const;

	/**
	 * Controls visibility for widgets for sphere meshes.
	 */
	EVisibility ShouldShowMeshSphereWidgets() const;

	/**
	 * Call this to switch between planes, spheres, etc.
	 */
	void SetMeshMode(EMediaTextureVisibleMipsTiles InMode);

	/**
	 * Call this to apply a sphere mesh to an object.
	 */
	void SetSphereMesh(UMediaPlateComponent* MediaPlate);

	/**
	 * Returns menu options for all aspect ratio presets.
	 */
	TSharedRef<SWidget> OnGetAspectRatios();

	/**
	 * Returns menu options for all aspect ratio presets.
	 */
	TSharedRef<SWidget> OnGetLetterboxAspectRatios();

	/**
	 * Returns the text to display for a given aspect ratio.
	 */
	FText OnGetAspectRatioText() const;
	
	/**
	 * Returns the text to display for a given letter box aspect ratio.
	 */
	FText OnGetLetterboxAspectRatioText() const;
	
	/**
	 * Adds menu options for all aspect ratio presets.
	 */
	void AddAspectRatiosToMenuBuilder(FMenuBuilder& MenuBuilder,
		void (FMediaPlateCustomization::*Func)(float));

	/**
	 * Call this to see if auto aspect ratio is enabled.
	 */
	ECheckBoxState IsAspectRatioAuto() const;

	/**
	 * Call this to enable/disable automatic aspect ratio.
	 */
	void SetIsAspectRatioAuto(ECheckBoxState State);

	/**
	 * Call this to set the aspect ratio.
	 */
	void SetAspectRatio(float AspectRatio);

	/**
	 * Call this to get the aspect ratio.
	 */	
	float GetAspectRatio() const;

	/**
	 * Call this to set the aspect ratio.
	 */
	void SetLetterboxAspectRatio(float AspectRatio);

	/**
	 * Call this to get the aspect ratio.
	 */
	float GetLetterboxAspectRatio() const;

	/**
	 * Call this to set the horizontal range of the mesh.
	 */
	void SetMeshHorizontalRange(float HorizontalRange);

	/**
	 * Call this to get the horizontal range of the mesh.
	 */
	TOptional<float> GetMeshHorizontalRange() const;

	/**
	 * Call this to set the vertical range of the mesh.
	 */
	void SetMeshVerticalRange(float VerticalRange);

	/**
	 * Call this to get the vertical range of the mesh.
	 */
	TOptional<float> GetMeshVerticalRange() const;

	/**
	 * Call this to set the range of the mesh.
	 */
	void SetMeshRange(FVector2D Range);

	/**
	 * Gets the object path for the static mesh.
	 */
	FString GetStaticMeshPath() const;

	/**
	 * Called when the static mesh changes.
	 */
	void OnStaticMeshChanged(const FAssetData& AssetData);

	/**
	 * Determines if the event state switch is allowed in at least one selected components.
	 */
	bool IsButtonEventAllowedForAnyPlate(EMediaPlateEventState InState) const;

	/**
	* Changes the state of selected media plates and broadcasts the event to the remote endpoints.
	*/
	void OnButtonEvent(EMediaPlateEventState InState);

	/**
	 * Called when the open media plate button is pressed.
	 */
	FReply OnOpenMediaPlate();

	/**
	 * Call this to stop all playback.
	 */
	void StopMediaPlates();

	TArray<TWeakObjectPtr<UMediaPlayer>> GetMediaPlayers() const;

	/**
	 * Evaluates if the given predicate is true for all players.
	 */
	bool IsTrueForAllPlayers(TFunctionRef<bool(const UMediaPlayer*)> InPredicate) const;
};
