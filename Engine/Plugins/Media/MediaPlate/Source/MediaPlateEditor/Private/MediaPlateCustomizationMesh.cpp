// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaPlateCustomizationMesh.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetUtils/CreateStaticMeshUtil.h"
#include "Components/StaticMeshComponent.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "LevelEditorViewport.h"
#include "MediaPlate.h"
#include "MediaPlateComponent.h"
#include "MediaPlateSphereGenerator.h"

#define LOCTEXT_NAMESPACE "FMediaPlateCustomizationMesh"

TMap<UStaticMesh*, int32> FMediaPlateCustomizationMesh::MeshRefCount;

void FMediaPlateCustomizationMesh::SetCustomMesh(UMediaPlateComponent* MediaPlate,
	UStaticMesh* StaticMesh)
{
	// Get static mesh component.
	UStaticMeshComponent* StaticMeshComponent = nullptr;
	if (MediaPlate != nullptr)
	{
		AActor* Owner = MediaPlate->GetOwner();
		AMediaPlate* MediaPlateActor = Cast<AMediaPlate>(Owner);
		if (MediaPlateActor != nullptr)
		{
			StaticMeshComponent = MediaPlateActor->StaticMeshComponent;
		}
	}

	if (StaticMeshComponent != nullptr)
	{
		// Apply this mesh.
		SetMesh(StaticMeshComponent, StaticMesh);
	}
}

void FMediaPlateCustomizationMesh::SetPlaneMesh(UMediaPlateComponent* MediaPlate)
{
	// Get plane mesh.
	UStaticMesh* Mesh = Cast<UStaticMesh>(StaticLoadObject(UStaticMesh::StaticClass(), NULL,
		TEXT("/MediaPlate/SM_MediaPlateScreen")));

	// Apply this mesh.
	SetCustomMesh(MediaPlate, Mesh);
}

void FMediaPlateCustomizationMesh::SetSphereMesh(UMediaPlateComponent* MediaPlate)
{
	// Get static mesh component.
	UStaticMeshComponent* StaticMeshComponent = nullptr;
	if (MediaPlate != nullptr)
	{
		AActor* Owner = MediaPlate->GetOwner();
		AMediaPlate* MediaPlateActor = Cast<AMediaPlate>(Owner);
		if (MediaPlateActor != nullptr)
		{
			StaticMeshComponent = MediaPlateActor->StaticMeshComponent;
		}
	}

	if (StaticMeshComponent != nullptr)
	{
		// Do we already have this mesh?
		FString AssetPath = GetAssetPath(MediaPlate);
		UStaticMesh* StaticMesh = Cast<UStaticMesh>(StaticLoadObject(UStaticMesh::StaticClass(), NULL,
			*AssetPath));
		if (StaticMesh == nullptr)
		{
			// Create mesh.
			FDynamicMesh3 NewMesh;
			GenerateSphereMesh(&NewMesh, MediaPlate);

			// Create asset.
			StaticMesh = CreateStaticMeshAsset(&NewMesh, AssetPath);
			if (StaticMesh != nullptr)
			{
				MeshRefCount.Add(StaticMesh, 1);
			}
		}
		else
		{
			// Is this one of our generated meshes?
			int32* CountPtr = MeshRefCount.Find(StaticMesh);
			if (CountPtr != nullptr)
			{
				// Make sure its not transient.
				UPackage* MeshPackage = StaticMesh->GetPackage();
				if (MeshPackage != nullptr)
				{
					MeshPackage->ClearFlags(RF_Transient);
				}

				// Increase reference count.
				(*CountPtr)++;
			}
		}

		// Apply mesh.
		SetMesh(StaticMeshComponent, StaticMesh);
	}
}

void FMediaPlateCustomizationMesh::SetMesh(UStaticMeshComponent* StaticMeshComponent, UStaticMesh* Mesh)
{
	if (StaticMeshComponent != nullptr)
	{
		FProperty* StaticMeshProperty = FindFieldChecked<FProperty>(UStaticMeshComponent::StaticClass(), "StaticMesh");
		StaticMeshComponent->PreEditChange(StaticMeshProperty);

		// Get existing mesh.
		TObjectPtr<UStaticMesh> OldMesh = StaticMeshComponent->GetStaticMesh();
		if (OldMesh != nullptr)
		{
			// Is this one of our generated meshes?
			int32* CountPtr = MeshRefCount.Find(OldMesh);
			if (CountPtr != nullptr)
			{
				// Update reference count.
				(*CountPtr)--;
				if (*CountPtr <= 0)
				{
					// No longer needed, so mark as transient.
					OldMesh->GetPackage()->SetFlags(RF_Transient);
				}
			}
		}

		// Apply mesh to component.
		StaticMeshComponent->SetStaticMesh(Mesh);
		StaticMeshComponent->SetRelativeScale3D(FVector::OneVector);

		// Call PostEditChangeProperty so it updates properly.
		FPropertyChangedEvent PropertyEvent(StaticMeshProperty);
		StaticMeshComponent->PostEditChangeProperty(PropertyEvent);

		// Invalidate the viewport so we can see the mesh change.
		if (GCurrentLevelEditingViewportClient != nullptr)
		{
			GCurrentLevelEditingViewportClient->Invalidate();
		}
	}
}

void FMediaPlateCustomizationMesh::GenerateSphereMesh(FDynamicMesh3* OutMesh,
	UMediaPlateComponent* MediaPlate)
{
	FMediaPlateSphereGenerator SphereGen;
	SphereGen.Radius = 50.0f;
	SphereGen.ThetaRange = FMath::DegreesToRadians(MediaPlate->GetMeshRange().X);
	SphereGen.PhiRange = FMath::DegreesToRadians(MediaPlate->GetMeshRange().Y);
	SphereGen.NumTheta = 64;
	SphereGen.NumPhi = 64;
	SphereGen.bPolygroupPerQuad = false;
	SphereGen.Generate();

	OutMesh->Copy(&SphereGen);
}

UStaticMesh* FMediaPlateCustomizationMesh::CreateStaticMeshAsset(FDynamicMesh3* Mesh, const FString& AssetPath)
{
	UStaticMesh* NewStaticMesh = nullptr;

	UE::AssetUtils::FStaticMeshAssetOptions AssetOptions;

	AssetOptions.NewAssetPath = AssetPath;
	AssetOptions.NumSourceModels = 1;
	
	AssetOptions.bEnableRecomputeNormals = false;
	AssetOptions.bEnableRecomputeTangents = false;
	AssetOptions.bGenerateNaniteEnabledMesh = false;
	AssetOptions.NaniteSettings.FallbackPercentTriangles = 1.0;

	AssetOptions.bCreatePhysicsBody = true;;
	AssetOptions.CollisionType = ECollisionTraceFlag::CTF_UseComplexAsSimple;

	AssetOptions.SourceMeshes.DynamicMeshes.Add(Mesh);

	UE::AssetUtils::FStaticMeshResults ResultData;
	UE::AssetUtils::ECreateStaticMeshResult AssetResult = UE::AssetUtils::CreateStaticMeshAsset(AssetOptions, ResultData);

	if (AssetResult == UE::AssetUtils::ECreateStaticMeshResult::Ok)
	{
		NewStaticMesh = ResultData.StaticMesh;
		NewStaticMesh->MarkPackageDirty();
		FAssetRegistryModule::AssetCreated(NewStaticMesh);
	}

	return NewStaticMesh;
}

FString FMediaPlateCustomizationMesh::GetAssetPath(UMediaPlateComponent* MediaPlate)
{
	// Add the horizontal range.
	FString ID_X = FString::SanitizeFloat(MediaPlate->GetMeshRange().X);
	ID_X.ReplaceCharInline(TCHAR('.'), TCHAR('_'));
	FString ID_Y = FString::SanitizeFloat(MediaPlate->GetMeshRange().Y);
	ID_Y.ReplaceCharInline(TCHAR('.'), TCHAR('_'));

	FString AssetPath = FString::Printf(TEXT("/Game/_MediaPlate/Sphere_%s_%s"), *ID_X, *ID_Y);

	return AssetPath;
}

#undef LOCTEXT_NAMESPACE
