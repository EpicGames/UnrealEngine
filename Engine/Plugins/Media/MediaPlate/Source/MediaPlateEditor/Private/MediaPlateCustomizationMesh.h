// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryBase.h"

PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMesh3);

class UMediaPlateComponent;
class UStaticMesh;
class UStaticMeshComponent;

/**
 * Implements mesh handling for FMediaPlateCustomization.
 */
class FMediaPlateCustomizationMesh
{
public:
	/**
	 * Call this to use a custom mesh on a media plate.
	 */
	void SetCustomMesh(UMediaPlateComponent* MediaPlate, UStaticMesh* StaticMesh);

	/**
	 * Call this to use a plane mesh on a media plate.
	 */
	void SetPlaneMesh(UMediaPlateComponent* MediaPlate);

	/**
	 * Call this to use a sphere mesh on a media plate.
	*/
	void SetSphereMesh(UMediaPlateComponent* MediaPlate);

private:

	/**
	 * Generic function to set a mesh on a StaticMeshComponent.
	 */
	void SetMesh(UStaticMeshComponent* StaticMeshComponent, UStaticMesh* Mesh);

	/**
	 * Creates a sphere mesh.
	 */
	void GenerateSphereMesh(FDynamicMesh3* OutMesh, UMediaPlateComponent* MediaPlate);

	/**
	 * Call this to create a static mesh asset for a mesh.
	 */
	UStaticMesh* CreateStaticMeshAsset(FDynamicMesh3* Mesh, const FString& AssetPath);

	/**
	 * Gets the path for the mesh asset.
	 */
	FString GetAssetPath(UMediaPlateComponent* MediaPlate);

	/** Reference count for our generated meshes. */
	static TMap<UStaticMesh*, int32> MeshRefCount;
};

