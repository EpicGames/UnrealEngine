// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaPlateEditorModule.h"

#include "Editor.h"
#include "EngineAnalytics.h"
#include "HAL/PlatformApplicationMisc.h"
#include "IPlacementModeModule.h"
#include "ISequencerModule.h"
#include "LevelEditor.h"
#include "MaterialList.h"
#include "Materials/Material.h"
#include "MediaPlate.h"
#include "MediaPlateComponent.h"
#include "MediaPlateCustomization.h"
#include "MediaPlateEditorStyle.h"
#include "MediaPlateResourceCustomization.h"
#include "MediaSoundComponent.h"
#include "MediaSource.h"
#include "MediaTexture.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Sequencer/MediaPlateTrackEditor.h"
#include "SLevelViewport.h"
#include "Subsystems/EditorAssetSubsystem.h"
#include "Widgets/SMediaPlateEditorMaterial.h"

#define LOCTEXT_NAMESPACE "MediaPlateEditorModule"

DEFINE_LOG_CATEGORY(LogMediaPlateEditor);

TSharedPtr<FMediaPlateEditorStyle> FMediaPlateEditorStyle::Singleton;

void FMediaPlateEditorModule::StartupModule()
{
	FMediaPlateEditorStyle::Get();

	RegisterPlacementModeItems();
	RegisterSectionMappings();

	// Register customizations.
	MediaPlateName = UMediaPlateComponent::StaticClass()->GetFName();
	MediaPlateResourceName = FMediaPlateResource::StaticStruct()->GetFName();

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomClassLayout(MediaPlateName,
		FOnGetDetailCustomizationInstance::CreateStatic(&FMediaPlateCustomization::MakeInstance));

	PropertyModule.RegisterCustomPropertyTypeLayout(MediaPlateResourceName,
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FMediaPlateResourceCustomization::MakeInstance));

	ISequencerModule& SequencerModule = FModuleManager::Get().LoadModuleChecked<ISequencerModule>("Sequencer");
	TrackEditorBindingHandle = SequencerModule.RegisterPropertyTrackEditor<FMediaPlateTrackEditor>();

	RegisterContextMenuExtender();

	// Add bottom extender for material item
	FMaterialList::OnAddMaterialItemViewExtraBottomWidget.AddLambda([](const TSharedRef<FMaterialItemView>& InMaterialItemView, UActorComponent* InCurrentComponent, IDetailLayoutBuilder& InDetailBuilder, TArray<TSharedPtr<SWidget>>& OutExtensions)
	{
		LLM_SCOPE_BYNAME("MediaPlate/MediaPlateEditor");
		OutExtensions.Add(SNew(SMediaPlateEditorMaterial, InMaterialItemView, InCurrentComponent));
	});

	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FMediaPlateEditorModule::OnPostEngineInit);
}

void FMediaPlateEditorModule::ShutdownModule()
{
	FMaterialList::OnAddMaterialItemViewExtraBottomWidget.RemoveAll(this);

	UnregisterContextMenuExtender();

	if (GEditor != nullptr)
	{
		UEditorAssetSubsystem* EditorAssetSubsystem = GEditor->GetEditorSubsystem<UEditorAssetSubsystem>();
		if (EditorAssetSubsystem != nullptr)
		{
			EditorAssetSubsystem->GetOnExtractAssetFromFile().RemoveAll(this);
		}

		GEditor->OnLevelActorAdded().RemoveAll(this);
	}

	ISequencerModule* SequencerModulePtr = FModuleManager::Get().GetModulePtr<ISequencerModule>("Sequencer");
	if (SequencerModulePtr)
	{
		SequencerModulePtr->UnRegisterTrackEditor(TrackEditorBindingHandle);
	}
	UnregisterPlacementModeItems();

	// Unregister customizations.
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.UnregisterCustomClassLayout(MediaPlateName);
	PropertyModule.UnregisterCustomPropertyTypeLayout(MediaPlateResourceName);
}

void FMediaPlateEditorModule::Tick(float DeltaTime)
{
	bool bIsMediaPlatePlaying = false;

	// Loop through all our plates.
	for (int Index = 0; Index < ActiveMediaPlates.Num(); ++Index)
	{
		// Get the player.
		TWeakObjectPtr<UMediaPlateComponent>& PlatePtr = ActiveMediaPlates[Index];
		TObjectPtr<UMediaPlateComponent> MediaPlate = PlatePtr.Get();
		bool bIsMediaPlateToBeRemoved = true;
		if (MediaPlate != nullptr)
		{
			// Update sound component.
			TObjectPtr<UMediaSoundComponent> SoundComponent = MediaPlate->SoundComponent;
			if (SoundComponent != nullptr)
			{
				SoundComponent->UpdatePlayer();
			}

			bIsMediaPlateToBeRemoved = false;
			// Are we playing something?
			if (MediaPlate->IsMediaPlatePlaying())
			{
				bIsMediaPlatePlaying = true;
			}
		}

		// Is this player gone?
		if (bIsMediaPlateToBeRemoved)
		{
			ActiveMediaPlates.RemoveAtSwap(Index);
			--Index;
		}
	}

	// Is anything playing?
	ForceRealTimeViewports(bIsMediaPlatePlaying);
}

void FMediaPlateEditorModule::MediaPlateStartedPlayback(TObjectPtr<UMediaPlateComponent> MediaPlate)
{
	if (MediaPlate != nullptr)
	{
		ActiveMediaPlates.AddUnique(MediaPlate);
	}
}

bool FMediaPlateEditorModule::RemoveMediaSourceFromDragDropCache(UMediaSource* MediaSource)
{
	const FString* Key = MapFileToMediaSource.FindKey(MediaSource);
	bool bIsInCache = Key != nullptr;
	if (bIsInCache)
	{
		MapFileToMediaSource.Remove(*Key);
	}
	return bIsInCache;
}

FMediaPlateEditorModule::FOnGetMediaPlateMaterialAssetPaths& FMediaPlateEditorModule::OnGetMediaPlateMaterialAssetPaths()
{
	return OnGetMediaPlateMaterialAssetPathsDelegate;
}

const FPlacementCategoryInfo* FMediaPlateEditorModule::GetMediaCategoryRegisteredInfo() const
{
	const IPlacementModeModule& PlacementModeModule = IPlacementModeModule::Get();
	const FName PlacementModeCategoryHandle = TEXT("MediaPlate");

	if (const FPlacementCategoryInfo* RegisteredInfo = 
		PlacementModeModule.GetRegisteredPlacementCategory(PlacementModeCategoryHandle))
	{
		return RegisteredInfo;
	}

	const ISlateStyle* Style = &FMediaPlateEditorStyle::Get().Get();
	if (Style != nullptr)
	{
		FPlacementCategoryInfo Info(
			LOCTEXT("MediaPlateCategoryName", "Media Plate"),
			FSlateIcon(Style->GetStyleSetName(), "ClassIcon.MediaPlate"),
			PlacementModeCategoryHandle,
			TEXT("MediaPlate"),
			26 // Determines where the category shows up in the list with respect to the others.
		);

		Info.ShortDisplayName = LOCTEXT("MediaPlateShortCategoryName", "Media");
		IPlacementModeModule::Get().RegisterPlacementCategory(Info);

		return PlacementModeModule.GetRegisteredPlacementCategory(PlacementModeCategoryHandle);
	}
	else
	{
		return nullptr;
	}
}

void FMediaPlateEditorModule::RegisterPlacementModeItems()
{
	auto RegisterPlaceActors = [this]() -> void
	{
		if (!GEditor)
		{
			return;
		}

		const FPlacementCategoryInfo* Info = GetMediaCategoryRegisteredInfo();

		if (!Info)
		{
			UE_LOGF(LogMediaPlateEditor, Warning, "Could not find or create Media Place Actor Category");
			return;
		}

		// Register the Checkerboard
		PlaceActors.Add(IPlacementModeModule::Get().RegisterPlaceableItem(Info->UniqueHandle, MakeShared<FPlaceableItem>(
			*AMediaPlate::StaticClass(),
			FAssetData(AMediaPlate::StaticClass()),
			NAME_None,
			NAME_None,
			TOptional<FLinearColor>(),
			TOptional<int32>(),
			LOCTEXT("PlacementMode", "MediaPlate")
		)));
	};

	if (FApp::CanEverRender())
	{
		if (GEngine && GEngine->IsInitialized())
		{
			RegisterPlaceActors();
		}
		else
		{
			FCoreDelegates::GetOnPostEngineInit().AddLambda(RegisterPlaceActors);
		}
	}
}

void FMediaPlateEditorModule::RegisterSectionMappings()
{
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	// Media Plate.
	TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("Object", "General", LOCTEXT("General", "General"));
	Section->AddCategory("Control");
	Section->AddCategory("MediaPlate");
}

void FMediaPlateEditorModule::UnregisterPlacementModeItems()
{
	if (IPlacementModeModule::IsAvailable())
	{
		IPlacementModeModule& PlacementModeModule = IPlacementModeModule::Get();

		for (TOptional<FPlacementModeID>& PlaceActor : PlaceActors)
		{
			if (PlaceActor.IsSet())
			{
				PlacementModeModule.UnregisterPlaceableItem(*PlaceActor);
			}
		}
	}

	PlaceActors.Empty();
}

void FMediaPlateEditorModule::OnPostEngineInit()
{
	if (GEditor)
	{
		UEditorAssetSubsystem* EditorAssetSubsystem = GEditor->GetEditorSubsystem<UEditorAssetSubsystem>();
		if (EditorAssetSubsystem != nullptr)
		{
			EditorAssetSubsystem->GetOnExtractAssetFromFile().AddRaw(this, &FMediaPlateEditorModule::ExtractAssetDataFromFiles);
		}

		GEditor->OnLevelActorAdded().AddRaw(this, &FMediaPlateEditorModule::OnLevelActorAdded);
	}
}

void FMediaPlateEditorModule::ExtractAssetDataFromFiles(const TArray<FString>& Files,
	TArray<FAssetData>& AssetDataArray)
{
	if (Files.Num() > 0)
	{
		// Do we have this already?
		UMediaSource* MediaSource = nullptr;
		TWeakObjectPtr<UMediaSource>* ExistingMediaSourcePtr = MapFileToMediaSource.Find(Files[0]);
		if (ExistingMediaSourcePtr != nullptr)
		{
			MediaSource = ExistingMediaSourcePtr->Get();
		}

		// If we dont have it then create one now.
		if (MediaSource == nullptr)
		{
			MediaSource = UMediaSource::SpawnMediaSourceForString(Files[0], GetTransientPackage());
			if (MediaSource != nullptr)
			{
				MapFileToMediaSource.Emplace(Files[0], MediaSource);
			}
		}

		// Return this via the array.
		if (MediaSource != nullptr)
		{
			FAssetData AssetData(MediaSource);
			AssetDataArray.Add(AssetData);
		}
	}
}

void FMediaPlateEditorModule::ForceRealTimeViewports(const bool bEnable)
{
	if (bIsRealTimeViewportsEnabled != bEnable)
	{
		bIsRealTimeViewportsEnabled = bEnable;

		// Go through viewports.
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
		TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule.GetFirstLevelEditor();
		if (LevelEditor.IsValid())
		{
			TArray<TSharedPtr<SLevelViewport>> Viewports = LevelEditor->GetViewports();
			for (const TSharedPtr<SLevelViewport>& ViewportWindow : Viewports)
			{
				if (ViewportWindow.IsValid())
				{
					FEditorViewportClient& Viewport = ViewportWindow->GetAssetViewportClient();
					const FText SystemDisplayName = LOCTEXT("RealtimeOverrideMessage", "Media Plate");
					if (bEnable)
					{
						Viewport.AddRealtimeOverride(bEnable, SystemDisplayName);
					}
					else
					{
						Viewport.RemoveRealtimeOverride(SystemDisplayName, false);
					}
				}
			}
		}
	}
}

TSharedRef<FExtender> FMediaPlateEditorModule::ExtendLevelViewportContextMenuForMediaPlate(const TSharedRef<FUICommandList> CommandList, TArray<AActor*> SelectedActors)
{
	TSharedPtr<FExtender> Extender = MakeShared<FExtender>();

	if (SelectedActors.Num() == 1)
	{
		AMediaPlate* MediaPlateActor = Cast<AMediaPlate>(SelectedActors[0]);
		if (IsValid(MediaPlateActor))
		{
			Extender->AddMenuExtension("ActorTypeTools", EExtensionHook::After, CommandList,
				FMenuExtensionDelegate::CreateLambda([MediaPlateActor](FMenuBuilder& MenuBuilder, AActor* SelectedActor)
					{
						const ISlateStyle* Style = &FMediaPlateEditorStyle::Get().Get();
						check(Style);

						FUIAction Action_CopyMediaTextureReference(
							FExecuteAction::CreateLambda([](AMediaPlate* InMediaPlateActor)
								{
									if (!IsValid(InMediaPlateActor))
									{
										return;
									}

									if (UMediaPlateComponent* MediaPlateComponent = InMediaPlateActor->MediaPlateComponent)
									{
										if (const UMediaTexture* MediaTexture = MediaPlateComponent->GetMediaTexture(0))
										{
											FPlatformApplicationMisc::ClipboardCopy(*FAssetData(MediaTexture).GetExportTextName());
										}
									}
								}
								, MediaPlateActor)
						);

						MenuBuilder.BeginSection("MediaPlate", LOCTEXT("MediaPlateHeading", "Media Plate"));
						MenuBuilder.AddMenuEntry(
							LOCTEXT("CopyMediaTextureReference", "Copy Media Texture Reference"),
							LOCTEXT("CopyMediaTextureReference_Tooltip", "Copy Embedded Media Texture Reference to Clipboard."),
							FSlateIcon(Style->GetStyleSetName(), "ClassIcon.MediaPlate"),
							Action_CopyMediaTextureReference
						);

						MenuBuilder.EndSection();
					},
					MediaPlateActor
				)
			);
		}
	}

	return Extender.ToSharedRef();
}

void FMediaPlateEditorModule::RegisterContextMenuExtender()
{
	// Extend the level viewport context menu to add an option to copy the object path.
	LevelViewportContextMenuRemoteControlExtender = FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors::CreateRaw(this, &FMediaPlateEditorModule::ExtendLevelViewportContextMenuForMediaPlate);

	FLevelEditorModule& LevelEditorModule = FModuleManager::Get().LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	TArray<FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors>& MenuExtenders = LevelEditorModule.GetAllLevelViewportContextMenuExtenders();
	MenuExtenders.Add(LevelViewportContextMenuRemoteControlExtender);
	MenuExtenderDelegateHandle = MenuExtenders.Last().GetHandle();
}

void FMediaPlateEditorModule::UnregisterContextMenuExtender()
{
	if (FModuleManager::Get().IsModuleLoaded("LevelEditor"))
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
		LevelEditorModule.GetAllLevelViewportContextMenuExtenders().RemoveAll(
			[&](const FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors& Delegate)
			{
				return Delegate.GetHandle() == MenuExtenderDelegateHandle;
			});
	}
}

void FMediaPlateEditorModule::OnLevelActorAdded(AActor* InActor)
{
	if (!InActor || InActor->HasAnyFlags(RF_Transient))
	{
		return;
	}

	if (FEngineAnalytics::IsAvailable() && InActor->GetClass() == AMediaPlate::StaticClass())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.MediaPlate.AddMediaPlateActorToLevel"));
	}
}

IMPLEMENT_MODULE(FMediaPlateEditorModule, MediaPlateEditor)

#undef LOCTEXT_NAMESPACE
