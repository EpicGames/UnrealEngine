// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaPlateEditorStyle.h"

#include "Brushes/SlateBorderBrush.h"
#include "Brushes/SlateBoxBrush.h"
#include "Brushes/SlateImageBrush.h"
#include "Fonts/SlateFontInfo.h"
#include "Interfaces/IPluginManager.h"
#include "Math/Vector2D.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"

/* FMediaPlateEditorStyle structors
 *****************************************************************************/

FMediaPlateEditorStyle::FMediaPlateEditorStyle()
	: FSlateStyleSet("MediaPlateEditorStyle")
{
	const FVector2D Icon8x8(8.0f, 8.0f);
	const FVector2D Icon12x12(12.0f, 12.0f);
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon40x40(40.0f, 40.0f);
	const FVector2D Icon64x64(64.0f, 64.0f);

	if (TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("MediaPlate")))
	{
		SetContentRoot(FPaths::Combine(Plugin->GetBaseDir(), TEXT("Resources")));
	}
	// Class icons.
	Set("ClassIcon.MediaPlate", new IMAGE_BRUSH_SVG(TEXT("Icons/MediaPlate"), Icon16x16));
	Set("ClassThumbnail.MediaPlate", new IMAGE_BRUSH_SVG(TEXT("Icons/MediaPlate"), Icon64x64));
	Set("ClassIcon.MediaPlateComponent", new IMAGE_BRUSH_SVG(TEXT("Icons/MediaPlate"), Icon16x16));
	Set("ClassThumbnail.MediaPlateComponent", new IMAGE_BRUSH_SVG(TEXT("Icons/MediaPlate"), Icon64x64));

	SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Media/MediaPlayerEditor/Content"));

	// buttons
	Set("MediaPlayerEditor.SourceButton", new IMAGE_BRUSH("btn_source_12x", Icon12x12));
	Set("MediaPlayerEditor.GoButton", new IMAGE_BRUSH("btn_go_12x", Icon12x12));
	Set("MediaPlayerEditor.ReloadButton", new IMAGE_BRUSH("btn_reload_12x", Icon12x12));
	Set("MediaPlayerEditor.SettingsButton", new IMAGE_BRUSH("btn_settings_16x", Icon12x12));

	// misc
	Set("MediaPlateEditor.DragDropBorder", new BOX_BRUSH("border_dragdrop", 0.5f));
	Set("MediaPlateEditor.MediaSourceOpened", new IMAGE_BRUSH("mediasource_opened", Icon8x8));

	// tabs
	Set("MediaPlateEditor.Tabs.Info", new IMAGE_BRUSH("tab_info_16x", Icon16x16));
	Set("MediaPlateEditor.Tabs.Media", new IMAGE_BRUSH("tab_media_16x", Icon16x16));
	Set("MediaPlateEditor.Tabs.Player", new IMAGE_BRUSH("tab_player_16x", Icon16x16));
	Set("MediaPlateEditor.Tabs.Playlist", new IMAGE_BRUSH("tab_playlist_16x", Icon16x16));
	Set("MediaPlateEditor.Tabs.Stats", new IMAGE_BRUSH("tab_stats_16x", Icon16x16));

	// toolbar icons
	Set("MediaPlateEditor.CloseMedia", new IMAGE_BRUSH("icon_eject_40x", Icon40x40));
	Set("MediaPlateEditor.CloseMedia.Small", new IMAGE_BRUSH("icon_eject_40x", Icon20x20));
	Set("MediaPlateEditor.ForwardMedia", new IMAGE_BRUSH("icon_forward_40x", Icon40x40));
	Set("MediaPlateEditor.ForwardMedia.Small", new IMAGE_BRUSH("icon_forward_40x", Icon20x20));
	Set("MediaPlateEditor.NextMedia", new IMAGE_BRUSH("icon_step_40x", Icon40x40));
	Set("MediaPlateEditor.NextMedia.Small", new IMAGE_BRUSH("icon_step_40x", Icon20x20));
	Set("MediaPlateEditor.OpenMedia", new IMAGE_BRUSH("icon_open_40x", Icon40x40));
	Set("MediaPlateEditor.OpenMedia.Small", new IMAGE_BRUSH("icon_open_40x", Icon20x20));
	Set("MediaPlateEditor.PauseMedia", new IMAGE_BRUSH("icon_pause_40x", Icon40x40));
	Set("MediaPlateEditor.PauseMedia.Small", new IMAGE_BRUSH("icon_pause_40x", Icon20x20));
	Set("MediaPlateEditor.PlayMedia", new IMAGE_BRUSH("icon_play_40x", Icon40x40));
	Set("MediaPlateEditor.PlayMedia.Small", new IMAGE_BRUSH("icon_play_40x", Icon20x20));
	Set("MediaPlateEditor.PreviousMedia", new IMAGE_BRUSH("icon_step_back_40x", Icon40x40));
	Set("MediaPlateEditor.PreviousMedia.Small", new IMAGE_BRUSH("icon_step_back_40x", Icon20x20));
	Set("MediaPlateEditor.ReverseMedia", new IMAGE_BRUSH("icon_reverse_40x", Icon40x40));
	Set("MediaPlateEditor.ReverseMedia.Small", new IMAGE_BRUSH("icon_reverse_40x", Icon20x20));
	Set("MediaPlateEditor.RewindMedia", new IMAGE_BRUSH("icon_rewind_40x", Icon40x40));
	Set("MediaPlateEditor.RewindMedia.Small", new IMAGE_BRUSH("icon_rewind_40x", Icon20x20));
	Set("MediaPlateEditor.StopMedia", new IMAGE_BRUSH("icon_stop_40x", Icon40x40));
	Set("MediaPlateEditor.StopMedia.Small", new IMAGE_BRUSH("icon_stop_40x", Icon20x20));

	// scrubber
	Set("MediaPlayerEditor.Scrubber", FSliderStyle()
		.SetNormalBarImage(FSlateColorBrush(FColor::White))
		.SetDisabledBarImage(FSlateColorBrush(FLinearColor::Gray))
		.SetNormalThumbImage(IMAGE_BRUSH("scrubber", FVector2D(2.0f, 10.0f)))
		.SetHoveredThumbImage(IMAGE_BRUSH("scrubber", FVector2D(2.0f, 10.0f)))
		.SetDisabledThumbImage(IMAGE_BRUSH("scrubber", FVector2D(2.0f, 10.0f)))
		.SetBarThickness(2.0f)
	);

	Set("MediaPlateEditor.ViewportFont", DEFAULT_FONT("Regular", 18));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FMediaPlateEditorStyle::~FMediaPlateEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}
