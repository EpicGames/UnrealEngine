// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

/**
 * Implements the visual style of the media plate editor UI.
 */
class FMediaPlateEditorStyle : public FSlateStyleSet
{
public:
	/** Default constructor. */
	FMediaPlateEditorStyle();

	 /** Destructor. */
	~FMediaPlateEditorStyle();

	static TSharedRef<FMediaPlateEditorStyle> Get()
	{
		if (!Singleton.IsValid())
		{
			Singleton = MakeShareable(new FMediaPlateEditorStyle);
		}
		return Singleton.ToSharedRef();
	}

private:
	static TSharedPtr<FMediaPlateEditorStyle> Singleton;
};
