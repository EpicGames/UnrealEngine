// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaPlateResourceCustomization.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "EditorDirectories.h"
#include "MediaPlateComponent.h"
#include "MediaPlaylist.h"
#include "MediaSource.h"
#include "PropertyCustomizationHelpers.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SFilePathPicker.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "MediaPlateResourceCustomization"

TSharedRef<IPropertyTypeCustomization> FMediaPlateResourceCustomization::MakeInstance()
{
	return MakeShared<FMediaPlateResourceCustomization>();
}

void FMediaPlateResourceCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InStructCustomizationUtils)
{
	const TAttribute<EVisibility> MediaSourceFileVisibility(this, &FMediaPlateResourceCustomization::GetFileSelectorVisibility);
	const TAttribute<EVisibility> MediaSourceAssetVisibility(this, &FMediaPlateResourceCustomization::GetAssetSelectorVisibility);
	const TAttribute<EVisibility> MediaSourcePlaylistVisibility(this, &FMediaPlateResourceCustomization::GetPlaylistSelectorVisibility);
	const TAttribute<EVisibility> MultipleValuesVisibility(this, &FMediaPlateResourceCustomization::GetMultipleValuesVisibility);

	MediaPlateResourcePropertyHandle = InStructPropertyHandle;
	ResourceTypePropertyHandle = MediaPlateResourcePropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMediaPlateResource, Type));
	ExternalMediaPathPropertyHandle = MediaPlateResourcePropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMediaPlateResource, ExternalMediaPath));
	MediaAssetPropertyHandle = MediaPlateResourcePropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMediaPlateResource, MediaAsset));
	SourcePlaylistPropertyHandle = MediaPlateResourcePropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMediaPlateResource, SourcePlaylist));

	void* StructRawData;
	// Try accessing the raw value, so we make sure there's only one struct edited. Multiple Access edit is currently not supported
	const FPropertyAccess::Result AccessResult = MediaPlateResourcePropertyHandle->GetValueData(StructRawData);

	TSharedPtr<SWidget> ValueWidgetContent;

	if (AccessResult == FPropertyAccess::Success || AccessResult == FPropertyAccess::MultipleValues)
	{
		ValueWidgetContent = SNew(SBox)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Left)
			.MinHeight(22)
			.MaxHeight(22)
			[
				SNew(SSegmentedControl<TOptional<EMediaPlateResourceType>>)
					.Value(this, &FMediaPlateResourceCustomization::GetAssetType)
					.OnValueChanged(this, &FMediaPlateResourceCustomization::OnAssetTypeChanged)

				+ SSegmentedControl<TOptional<EMediaPlateResourceType>>::Slot(EMediaPlateResourceType::External)
					.Text(LOCTEXT("File", "File"))
					.ToolTip(LOCTEXT("File_ToolTip",
						"Select this if you want to use a file path to a media file on disk."))

				+ SSegmentedControl<TOptional<EMediaPlateResourceType>>::Slot(EMediaPlateResourceType::Asset)
					.Text(LOCTEXT("Asset", "Asset"))
					.ToolTip(LOCTEXT("Asset_ToolTip",
						"Select this if you want to use a Media Source asset."))

				+ SSegmentedControl<TOptional<EMediaPlateResourceType>>::Slot(EMediaPlateResourceType::Playlist)
					.Text(LOCTEXT("Playlist", "Playlist"))
					.ToolTip(LOCTEXT("Playlist_ToolTip",
						"Select this if you want to use a Media Playlist asset."))
			]
			+ SVerticalBox::Slot()
			[
				SNew(SBox)
				.Visibility(MediaSourceAssetVisibility)
				.HAlign(HAlign_Fill)
				[
					SNew(SObjectPropertyEntryBox)
					.AllowedClass(UMediaSource::StaticClass())
					.AllowCreate(true)
					.PropertyHandle(MediaAssetPropertyHandle)
				]
			]
			+ SVerticalBox::Slot()
			[
				SNew(SBox)
				.Visibility(MediaSourceFileVisibility)
				.HAlign(HAlign_Fill)
				[
					SNew(SFilePathPicker)
					.BrowseButtonImage(FAppStyle::GetBrush("PropertyWindow.Button_Ellipsis"))
					.BrowseButtonStyle(FAppStyle::Get(), "HoverHintOnly")
					.BrowseButtonToolTip(LOCTEXT("FileButtonToolTipText", "Choose a file from this computer"))
					.BrowseDirectory(this, &FMediaPlateResourceCustomization::GetMediaBrowseDirectory)
					.BrowseTitle(LOCTEXT("PropertyEditorTitle", "File picker..."))
					.FilePath(this, &FMediaPlateResourceCustomization::GetMediaPath)
					.FileTypeFilter(TEXT("All files (*.*)|*.*"))
					.OnPathPicked(this, &FMediaPlateResourceCustomization::OnMediaPathPicked)
				]
			]
			+ SVerticalBox::Slot()
			[
				SNew(SBox)
				.Visibility(MediaSourcePlaylistVisibility)
				.HAlign(HAlign_Fill)
				[
					SNew(SObjectPropertyEntryBox)
					.AllowedClass(UMediaPlaylist::StaticClass())
					.AllowCreate(true)
					.PropertyHandle(SourcePlaylistPropertyHandle)
				]
			]
			+ SVerticalBox::Slot()
			[
				SNew(SBox)
				.Visibility(MultipleValuesVisibility)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				.Padding(0, 4)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("MultipleValues", "Multiple Values"))
					.ToolTipText(LOCTEXT("MultipleValues_ToolTip", "Multiple Values can't be displayed."))
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
			]
		];
	}
	else
	{
		ValueWidgetContent = SNew(STextBlock)
			.Text(LOCTEXT("AccessError", "Error accessing property"))
			.ToolTipText(LOCTEXT("AccessError_ToolTip",
			"Error occurred while accessing Media Player Resource property."))
			.Font(IDetailLayoutBuilder::GetDetailFont());
	}

	InHeaderRow.NameContent()
	[
		MediaPlateResourcePropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		ValueWidgetContent.ToSharedRef()
	];
}

void FMediaPlateResourceCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
}

TOptional<EMediaPlateResourceType> FMediaPlateResourceCustomization::GetAssetType() const
{
	if (ResourceTypePropertyHandle)
	{
		using UnderlyingType = std::underlying_type_t<EMediaPlateResourceType>;
		UnderlyingType ResourceType;
		if (ResourceTypePropertyHandle->GetValue(ResourceType) == FPropertyAccess::Success)
		{
			return static_cast<EMediaPlateResourceType>(ResourceType);
		}
	}
	return TOptional<EMediaPlateResourceType>();
}

void FMediaPlateResourceCustomization::OnAssetTypeChanged(TOptional<EMediaPlateResourceType> InMediaSourceType)
{
	if (ResourceTypePropertyHandle && InMediaSourceType.IsSet())
	{
		using UnderType = std::underlying_type_t<EMediaPlateResourceType>;
		ResourceTypePropertyHandle->SetValue(*reinterpret_cast<UnderType*>(&InMediaSourceType));
	}
}

FString FMediaPlateResourceCustomization::GetMediaPath() const
{
	if (ExternalMediaPathPropertyHandle)
	{
		FString Path;
		const FPropertyAccess::Result AccessResult = ExternalMediaPathPropertyHandle->GetValue(Path);
		if (AccessResult == FPropertyAccess::Success)
		{
			return Path;
		}
		if (AccessResult == FPropertyAccess::MultipleValues)
		{
			return TEXT("(Multiple values)");
		}
	}
	return FString();
}

namespace UE::MediaPlateResourceCustomization::Private
{
	/**
	 * Returns list of possible media base paths in order of priority. 
	 */
	const TArray<FString>& GetPossibleBasePaths()
	{
		static const TArray<FString> PossibleBasePaths
		{
			FPaths::ProjectContentDir(),
			FPaths::ProjectDir()
		};
		return PossibleBasePaths;
	}

	FString EnsureStartWithDotSlash(const FString& InPath)
	{
		// Note: exception for tokens that start with "{".		
		if (!InPath.StartsWith(TEXT("./")) && !InPath.StartsWith(TEXT("{")))
		{
			return FPaths::Combine(TEXT("."), InPath);
		}
		return InPath;
	}

	/** Converts the given path relative to one of the possible base paths. */
	bool TryConvertAbsoluteToRelative(FString& InAbsolutePath)
	{
		FString ConvertedPath = InAbsolutePath;

		for (const FString& BasePath : GetPossibleBasePaths())
		{
			const FString FullBasePath = FPaths::ConvertRelativePathToFull(BasePath);
			if (FPaths::IsUnderDirectory(ConvertedPath, FullBasePath) && FPaths::MakePathRelativeTo(ConvertedPath, *FullBasePath)) 
			{
				InAbsolutePath = EnsureStartWithDotSlash(ConvertedPath);
				return true;
			}
		}
		return false;
	}
	
	/**
	 * Returns a sanitized path compliant with the path resolution rules
	 * of ImgMediaSource and FileMediaSource.
	 * 
	 * @param InPickedPath Picked path to an existing file.
	 * @return sanitized path 
	 */
	FString SanitizePickedPath(const FString& InPickedPath)
	{
		if (InPickedPath.IsEmpty())
		{
			return InPickedPath;	
		}

		FString NormalizedPath = InPickedPath.TrimStartAndEnd().Replace(TEXT("\""), TEXT(""));	
		FPaths::NormalizeDirectoryName(NormalizedPath);

		if (FPaths::IsRelative(NormalizedPath))
		{
			// 1- Try to resolve as relative to BaseDir...
			FString FullMediaPath = FPaths::ConvertRelativePathToFull(NormalizedPath);
			if (FPaths::FileExists(FullMediaPath))
			{
				// Convert absolute path to relative (if possible), leave absolute if not.
				TryConvertAbsoluteToRelative(FullMediaPath);
				return FullMediaPath;
			}

			// 2- Try to find under which possible base path this path is relative to.
			for (const FString& BasePath : GetPossibleBasePaths())
			{
				const FString FullBasePath = FPaths::ConvertRelativePathToFull(BasePath);
				FString CombinedMediaPath = FPaths::Combine(FullBasePath, NormalizedPath);
				
				if (FPaths::FileExists(CombinedMediaPath) && FPaths::MakePathRelativeTo(CombinedMediaPath, *FullBasePath))
				{
					return EnsureStartWithDotSlash(CombinedMediaPath);
				}
			}

			// 3- Couldn't find a base, leave as is, but make sure it has a ./
			return EnsureStartWithDotSlash(NormalizedPath);
		}

		// Convert absolute path to relative (if possible), leave absolute if not.
		TryConvertAbsoluteToRelative(NormalizedPath);
		return NormalizedPath;
	}
}

FString FMediaPlateResourceCustomization::GetMediaBrowseDirectory() const
{
	const FString MediaPath = GetMediaPath();
	if (!MediaPath.IsEmpty())
	{
		const FString MediaDirectory = FPaths::GetPath(MediaPath);
		
		if (FPaths::DirectoryExists(MediaDirectory))
		{
			return MediaDirectory;
		}
		
		if (FPaths::IsRelative(MediaDirectory))
		{
			for (const FString& BasePath : UE::MediaPlateResourceCustomization::Private::GetPossibleBasePaths())
			{
				FString ExpandedDirectory = FPaths::ConvertRelativePathToFull(FPaths::Combine(BasePath, MediaDirectory));
				if (FPaths::DirectoryExists(ExpandedDirectory))
				{
					return ExpandedDirectory;
				}
			}
		}
	}

	// Fallback to last opened directory.
	return FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_OPEN);
}

void FMediaPlateResourceCustomization::OnMediaPathPicked(const FString& InPickedPath)
{
	if (ExternalMediaPathPropertyHandle)
	{
		const FString SanitizedMediaPath = UE::MediaPlateResourceCustomization::Private::SanitizePickedPath(InPickedPath);
		ExternalMediaPathPropertyHandle->SetValue(SanitizedMediaPath);
		FEditorDirectories::Get().SetLastDirectory(ELastDirectory::GENERIC_OPEN, FPaths::GetPath(InPickedPath));
	}
}

EVisibility FMediaPlateResourceCustomization::GetAssetSelectorVisibility() const
{
	return GetAssetType() == EMediaPlateResourceType::Asset ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FMediaPlateResourceCustomization::GetFileSelectorVisibility() const
{
	return GetAssetType() == EMediaPlateResourceType::External ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FMediaPlateResourceCustomization::GetPlaylistSelectorVisibility() const
{
	return GetAssetType() == EMediaPlateResourceType::Playlist ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FMediaPlateResourceCustomization::GetMultipleValuesVisibility() const
{
	return GetAssetType().IsSet() ? EVisibility::Collapsed : EVisibility::Visible;
}

#undef LOCTEXT_NAMESPACE
