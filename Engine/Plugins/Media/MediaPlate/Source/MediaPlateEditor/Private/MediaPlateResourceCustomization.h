// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "MediaPlateComponent.h"

class IPropertyHandle;
struct EVisibility;
struct FMediaPlateResource;

class FMediaPlateResourceCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	//~ Begin IPropertyTypeCustomization
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle
		, FDetailWidgetRow& InHeaderRow
		, IPropertyTypeCustomizationUtils& InStructCustomizationUtils) override;

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle
		, IDetailChildrenBuilder& InChildBuilder
		, IPropertyTypeCustomizationUtils& InCustomizationUtils) override;
	//~ End IPropertyTypeCustomization

private:
	TOptional<EMediaPlateResourceType> GetAssetType() const;

	FString GetMediaPath() const;
	FString GetMediaBrowseDirectory() const;

	void OnAssetTypeChanged(TOptional<EMediaPlateResourceType> InResourceType);
	void OnMediaPathPicked(const FString& InPickedPath);

	EVisibility GetAssetSelectorVisibility() const;
	EVisibility GetFileSelectorVisibility() const;
	EVisibility GetPlaylistSelectorVisibility() const;
	EVisibility GetMultipleValuesVisibility() const;


	TSharedPtr<IPropertyHandle> ResourceTypePropertyHandle;
	TSharedPtr<IPropertyHandle> MediaPlateResourcePropertyHandle;
	TSharedPtr<IPropertyHandle> ExternalMediaPathPropertyHandle;
	TSharedPtr<IPropertyHandle> MediaAssetPropertyHandle;
	TSharedPtr<IPropertyHandle> SourcePlaylistPropertyHandle;
};
