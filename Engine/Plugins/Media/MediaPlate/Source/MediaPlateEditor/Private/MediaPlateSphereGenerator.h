// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Generators/MeshShapeGenerator.h"
#include "OrientedBoxTypes.h"
#include "Util/IndexUtil.h"

/**
 * Generate a sphere mesh, with UVs wrapped cylindrically
 */
class FMediaPlateSphereGenerator : public UE::Geometry::FMeshShapeGenerator
{
public:
	/** Radius */
	double Radius = 1;
	/** Range for the vertical arc. 180 degrees for a sphere, etc... */
	double PhiRange = FMathd::Pi;
	/** Range for the horizontal arc. 360 degrees for a sphere, etc... */
	double ThetaRange = FMathd::TwoPi;

	int NumPhi = 3; // number of vertices along vertical extent from north pole to south pole
	int NumTheta = 3; // number of vertices around circles

	/** If true, each quad of box gets a separate polygroup */
	bool bPolygroupPerQuad = false;

private:
	/** True if this is a closed mesh, e.g. a sphere. */
	bool bIsClosed = false;
	/** True if we have the north and south poles. */
	bool bHasPoles = false;

	static FVector3d SphericalToCartesian(double r, double theta, double phi)
	{
		double Sphi = sin(phi);
		double Cphi = cos(phi);
		double Ctheta = cos(theta);
		double Stheta = sin(theta);

		return FVector3d(r * Ctheta * Sphi, r * Stheta * Sphi, r * Cphi);
	}

	void GenerateVertices()
	{
		auto SetVertex = [this](int32 VtxIdx,
								FVector3d Pos, FVector3f Normal )
		{
			Vertices[VtxIdx] = Pos;
			Normals[VtxIdx] = Normal;
			NormalParentVertex[VtxIdx] = VtxIdx;
		};
		{
			int32 ThetaDivide = bIsClosed ? NumTheta : NumTheta -1;
			const double Dphi = PhiRange / double(NumPhi - 1);
			const double Dtheta = ThetaRange / double(ThetaDivide);

			int32 p,t;
			double Phi, Theta;
			int32 VtxIdx = 0;

			// Add points between the poles
			int32 pStart = 1;
			int32 pEnd = NumPhi - 1;
			double PhiStart = Dphi;

			// If we dont have poles then add another layer top/bottom.
			if (bHasPoles == false)
			{
				pStart--;
				pEnd++;
				PhiStart = (FMathd::Pi - PhiRange) * 0.5;
			}

			for (p = pStart, Phi = PhiStart; p < pEnd; ++p, Phi += Dphi) // NB: this skips the poles.
			{
				for (t = 0, Theta = 0; t < NumTheta; ++t, ++VtxIdx, Theta += Dtheta)
				{
					FVector3d Normal = SphericalToCartesian(1., Theta, Phi);
					SetVertex(VtxIdx, Normal * Radius, FVector3f(-Normal));
				}
			}

			if (bHasPoles)
			{
				// add a single point at the North Pole
				SetVertex(VtxIdx++, FVector3d::UnitZ() * Radius, -FVector3f::UnitZ());
				// add a single point at the South Pole
				SetVertex(VtxIdx++, -FVector3d::UnitZ() * Radius, FVector3f::UnitZ());
			}
		}
	}

	void GenerateUVVertices()
	{
		// generate the UV's
		int32 ThetaDivide = bIsClosed ? NumTheta : NumTheta - 1;
		const float DUVphi = 1.0f / float(NumPhi - 1);
		const float DUVtheta = 1.0f / float(ThetaDivide);

		int32 UVIdx = 0;
		int32 p,t;
		float UVPhi, UVTheta;
		float UVPhiStart = DUVphi;
		int32 pStart = 1;
		int32 pEnd = NumPhi - 1;

		// If we dont have poles then add another layer top/bottom.
		if (bHasPoles == false)
		{
			UVPhiStart = 0.0f;
			pStart--;
			pEnd++;
		}
		
		for ( p = pStart, UVPhi = UVPhiStart; p < pEnd; ++p, UVPhi += DUVphi)
		{
			for (t = 0, UVTheta = 0; t < NumTheta; ++t, ++UVIdx, UVTheta += DUVtheta)
			{
				UVs[UVIdx] = FVector2f(UVTheta, UVPhi);
				UVParentVertex[UVIdx] = (p - 1) * NumTheta + t;
			}
			UVs[UVIdx] = FVector2f(UVTheta, UVPhi);
			UVParentVertex[UVIdx] = (p - 1) * NumTheta; // Wrap around
			++UVIdx;
		}

		if (bHasPoles)
		{
			int32 NorthPoleVtxIdx = (NumPhi - 2) * NumTheta;
			for (t = 0, UVTheta = DUVtheta; t < NumTheta; ++t, ++UVIdx, UVTheta += DUVtheta)
			{
				UVs[UVIdx] = FVector2f(UVTheta, 0.0);
				UVParentVertex[UVIdx] = NorthPoleVtxIdx;
			}
			int32 SouthPoleVtxIdx = NorthPoleVtxIdx + 1;
			for (t = 0, UVTheta = DUVtheta; t < NumTheta; ++t, ++UVIdx, UVTheta += DUVtheta)
			{
				UVs[UVIdx] = FVector2f(UVTheta, 1.0);
				UVParentVertex[UVIdx] = SouthPoleVtxIdx;
			}
		}
	}

	using CornerIndices =  UE::Geometry::FVector3i;
	void OutputTriangle(int TriIdx, int PolyIdx,  CornerIndices Corners, CornerIndices UVCorners)
	{
		SetTriangle(TriIdx, Corners.Z, Corners.Y, Corners.X);
		SetTrianglePolygon(TriIdx, PolyIdx);
		SetTriangleUVs(TriIdx, UVCorners.Z, UVCorners.Y, UVCorners.X);
		SetTriangleNormals(TriIdx, Corners.Z, Corners.Y, Corners.X);
	}

	void OutputEquatorialTriangles()
	{
		int32 TriIdx = 0, PolyIdx = 0;

		// Generate equatorial triangles
		int32 Corners[4] =   { 0, 1,     NumTheta + 1, NumTheta};
		int32 UVCorners[4] = { 0, 1, NumTheta + 2, NumTheta + 1};
		int32 pStart = 1;
		int32 pEnd = NumPhi - 2;

		// If we dont have poles then add another layer top/bottom.
		if (bHasPoles == false)
		{
			pStart--;
			pEnd++;
		}

		for (int32 p = pStart; p < pEnd; ++p)
		{
			for (int32 t = 0; t < NumTheta - 1; ++t)
			{
				// convert each quad into 2 triangles.
				OutputTriangle(TriIdx++, PolyIdx,
							   {Corners[0],   Corners[1],   Corners[2]},
							   {UVCorners[0], UVCorners[1], UVCorners[2]});
				OutputTriangle(TriIdx++, PolyIdx,
							   {Corners[2],   Corners[3],   Corners[0]},
							   {UVCorners[2], UVCorners[3], UVCorners[0]});
				for (auto& i : Corners) ++i; 
				for (auto& i : UVCorners) ++i;
				if (bPolygroupPerQuad)
				{
					PolyIdx++;
				}
			}
			// Close up this mesh if required.
			if (bIsClosed)
			{
				OutputTriangle(TriIdx++, PolyIdx,
					{Corners[0], Corners[1] - NumTheta, Corners[2] - NumTheta},
					{UVCorners[0]         , UVCorners[1],            UVCorners[2]});
				OutputTriangle(TriIdx++, PolyIdx,
					{Corners[2] - NumTheta, Corners[3],   Corners[0]},
					{UVCorners[2],          UVCorners[3],            UVCorners[0]});
			}
			for (auto& i : Corners) ++i; 
			for (auto& i : UVCorners) i += 2;
			if (bPolygroupPerQuad)
			{
				PolyIdx++;
			}
		}
	}

	void OutputPolarTriangles()
	{
		const int32 NumEquatorialVtx = (NumPhi - 2) * NumTheta;
		const int32 NumEquatorialUVVtx = (NumPhi - 2) * (NumTheta + 1);
		const int32 NorthPoleVtxIdx = NumEquatorialVtx;
		const int32 SouthPoleVtxIdx = NumEquatorialVtx + 1;
		int32 PolyIdx = NumTheta  * (NumPhi - 3);
		// If the mesh is not closed then we have less tris.
		if (bIsClosed == false)
		{
			PolyIdx -= (NumPhi - 3);
		}
		int32 TriIdx = PolyIdx * 2;
		if (bPolygroupPerQuad == false)
		{
			PolyIdx = 0;
		}

		// Triangles that connect to north pole
		int32 NumTris = bIsClosed ? NumTheta : NumTheta - 1;
		for (int32 t = 0; t < NumTris; ++t)
		{
			OutputTriangle(TriIdx++, PolyIdx,
						   {t, NorthPoleVtxIdx,       (t + 1) % NumTheta},
						   {t, NumEquatorialUVVtx + t, t + 1});
			if (bPolygroupPerQuad)
			{
				PolyIdx++;
			}
		}

		// Triangles that connect to South pole
		const int32 Offset   = NumEquatorialVtx - NumTheta;
		const int32 OffsetUV = NumEquatorialUVVtx - (NumTheta + 1);
		for (int32 t = 0; t < NumTris; ++t)
		{
			OutputTriangle(TriIdx++, PolyIdx,
						   {t + Offset,   ((t + 1) % NumTheta) + Offset, SouthPoleVtxIdx},
						   {t + OffsetUV, t + 1 + OffsetUV             , NumEquatorialUVVtx + NumTheta + t});
			if (bPolygroupPerQuad)
			{
				PolyIdx++;
			}
		}
	}
public:
	/** Generate the mesh */
	FMediaPlateSphereGenerator& Generate() override
	{
		// Is this a closed sphere?
		bIsClosed = ThetaRange >= FMath::DegreesToRadians(359.9f);
		if (bIsClosed)
		{
			ThetaRange = FMathd::TwoPi;
		}

		// Do we have the poles?
		bHasPoles = PhiRange >= FMath::DegreesToRadians(179.9f);
		if (bHasPoles)
		{
			PhiRange = FMathd::Pi;
		}
		
		// enforce sane values for vertex counts
		NumPhi = FMath::Max(NumPhi, 3);
		NumTheta = FMath::Max(NumTheta, 3);
		int32 NumVertices = (NumPhi - 2) * NumTheta + 2;
		int32 NumUVs = (NumPhi - 2) * (NumTheta + 1) + (2 * NumTheta);
		int32 NumTris = (NumPhi - 2) * (bIsClosed ? NumTheta : (NumTheta - 1)) * 2;
		if (bHasPoles == false)
		{
			NumVertices = NumPhi * NumTheta;
			NumUVs = NumPhi * (NumTheta + 1);
			NumTris = (NumPhi - 1) * (bIsClosed ? NumTheta : (NumTheta - 1)) * 2;
		}
		SetBufferSizes(NumVertices, NumTris, NumUVs, NumVertices);

		GenerateVertices();
		GenerateUVVertices();
		OutputEquatorialTriangles();
		if (bHasPoles)
		{
			OutputPolarTriangles();
		}
		return *this;
	}


};
