// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sequencer/MediaPlateTrackEditor.h"

#include "ActorTreeItem.h"
#include "EngineAnalytics.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ISequencer.h"
#include "MediaTexture.h"
#include "MediaPlate.h"
#include "MediaPlateComponent.h"
#include "MediaPlaylist.h"
#include "Misc/ITransaction.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "MovieSceneObjectBindingID.h"
#include "MovieSceneMediaSection.h"
#include "MovieSceneMediaTrack.h"
#include "SceneOutlinerModule.h"
#include "Sequencer/MediaTrackEditor.h"
#include "SequencerSettings.h"

#define LOCTEXT_NAMESPACE "FMediaPlateTrackEditor"

/* FMediaTrackEditor static functions
 *****************************************************************************/

TArray<FAnimatedPropertyKey, TInlineAllocator<1>> FMediaPlateTrackEditor::GetAnimatedPropertyTypes()
{
	return TArray<FAnimatedPropertyKey, TInlineAllocator<1>>({ FAnimatedPropertyKey::FromObjectType(UMediaTexture::StaticClass()) });
}

/* FMediaTrackEditor structors
 *****************************************************************************/

FMediaPlateTrackEditor::FMediaPlateTrackEditor(TSharedRef<ISequencer> InSequencer)
	: FMovieSceneTrackEditor(InSequencer)
{
	OnActorAddedToSequencerHandle = InSequencer->OnActorAddedToSequencer().AddRaw(this, &FMediaPlateTrackEditor::HandleActorAdded);
	FMediaTrackEditor::OnBuildOutlinerEditWidget.AddRaw(this,
		&FMediaPlateTrackEditor::OnBuildOutlinerEditWidget);
}

FMediaPlateTrackEditor::~FMediaPlateTrackEditor()
{
	FMediaTrackEditor::OnBuildOutlinerEditWidget.RemoveAll(this);
}

FText FMediaPlateTrackEditor::GetDisplayName() const
{
	return LOCTEXT("MediaPlateTrackEditor_DisplayName", "Media Plate");
}

void FMediaPlateTrackEditor::BuildObjectBindingContextMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass)
{
	if ((ObjectClass != nullptr) && (ObjectClass->IsChildOf(AMediaPlate::StaticClass())))
	{
		MenuBuilder.BeginSection("Media Plate", LOCTEXT("MediaPlate", "Media Plate"));

		MenuBuilder.AddMenuEntry(
			LOCTEXT("Import", "Import"),
			LOCTEXT("ImportTooltip", "Import tracks from the Media Plate object."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateRaw(this,
				&FMediaPlateTrackEditor::ImportObjectBinding, ObjectBindings)));
	
		MenuBuilder.EndSection();
	}
}

void FMediaPlateTrackEditor::BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass)
{
	// Is this a media plate?
	if (ObjectClass != nullptr)
	{
		if (ObjectClass->IsChildOf(AMediaPlate::StaticClass()))
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("AddTrack", "Media"),
				LOCTEXT("AddAttachedTooltip", "Adds a media track attached to the object.\nOnly one track per binding is supported. Add Media Sections instead."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateRaw(this, &FMediaPlateTrackEditor::HandleAddMediaTrackToObjectBindingMenuEntryExecute, ObjectBindings),
					FCanExecuteAction::CreateRaw(this, &FMediaPlateTrackEditor::HandleAddMediaTrackToObjectBindingMenuEntryCanExecute, ObjectBindings)));
		}
	}
}

bool FMediaPlateTrackEditor::SupportsType(TSubclassOf<UMovieSceneTrack> TrackClass) const
{
	return false;
}

bool FMediaPlateTrackEditor::HandleAddMediaTrackToObjectBindingMenuEntryCanExecute(TArray<FGuid> InObjectBindingID)
{
	const TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	const UMovieSceneSequence* Sequence = SequencerPtr ? SequencerPtr->GetFocusedMovieSceneSequence() : nullptr; 
	const UMovieScene* MovieScene = Sequence ? Sequence->GetMovieScene() : nullptr;

	if (!MovieScene)
	{
		return false;
	}
	
	// Only allow adding a media track under this object binding if it doesn't already have one.
	for (const FGuid& ObjectBindingID : InObjectBindingID)
	{
		if (const FMovieSceneBinding* Binding = MovieScene->FindBinding(ObjectBindingID))
		{
			for (const UMovieSceneTrack* Track : Binding->GetTracks())
			{
				if (Track && Track->IsA<UMovieSceneMediaTrack>())
				{
					return false;
				}
			}
		}
	}
	return true;
}

void FMediaPlateTrackEditor::HandleAddMediaTrackToObjectBindingMenuEntryExecute(TArray<FGuid> InObjectBindingIDs)
{
	UMovieScene* FocusedMovieScene = GetFocusedMovieScene();

	if (FocusedMovieScene == nullptr)
	{
		return;
	}

	if (FocusedMovieScene->IsReadOnly())
	{
		return;
	}

	const FScopedTransaction Transaction(NSLOCTEXT("Sequencer", "AddMediaTrack_Transaction", "Add Media Track"));
	FocusedMovieScene->Modify();

	// Loop through all objects.
	for (FGuid InObjectBindingID : InObjectBindingIDs)
	{
		if (InObjectBindingID.IsValid())
		{
			// Add media track.			
			if (UMovieSceneMediaTrack* NewObjectTrack = FocusedMovieScene->AddTrack<UMovieSceneMediaTrack>(InObjectBindingID))
			{
				NewObjectTrack->SetDisplayName(LOCTEXT("MediaTrackName", "Media"));

				if (GetSequencer().IsValid())
				{
					GetSequencer()->OnAddTrack(NewObjectTrack, InObjectBindingID);
				}
			}
		}
	}
}

void FMediaPlateTrackEditor::HandleActorAdded(AActor* Actor, FGuid TargetObjectGuid)
{
	if (Actor)
	{
		if (FEngineAnalytics::IsAvailable() && Actor->GetClass() == AMediaPlate::StaticClass())
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.MediaPlate.AddMediaPlateActorToSequencer"));
		}

		if (UMediaPlateComponent* MediaPlateComponent = Actor->FindComponentByClass<UMediaPlateComponent>())
		{
			AddTrackForComponent(MediaPlateComponent, TargetObjectGuid);
		}
	}
}

void FMediaPlateTrackEditor::AddTrackForComponent(UMediaPlateComponent* Component, FGuid TargetObjectGuid)
{
	FMovieSceneObjectBindingID TargetObjectBindingID =
		UE::MovieScene::FRelativeObjectBindingID(TargetObjectGuid);

	// Get object.
	UObject* Object = Component->GetOwner();
	FFindOrCreateHandleResult HandleResult = FindOrCreateHandleToObject(Object);
	FGuid ObjectHandle = HandleResult.Handle;

	// Add media track.
	FFindOrCreateTrackResult TrackResult = FindOrCreateTrackForObject(ObjectHandle, UMovieSceneMediaTrack::StaticClass());
	UMovieSceneTrack* Track = TrackResult.Track;
	if (Track != nullptr)
	{
		UMovieSceneMediaTrack* MediaTrack = Cast<UMovieSceneMediaTrack>(Track);
		MediaTrack->SetDisplayName(LOCTEXT("MediaTrackName", "Media"));

		// Populate track.
		UMediaPlaylist* Playlist = Component->GetMediaPlaylist();
		if (Playlist != nullptr)
		{
			for (int32 Index = 0; Index < Playlist->Num(); ++Index)
			{
				UMediaSource* MediaSource = Playlist->Get(Index);
				if (MediaSource != nullptr)
				{
					UMovieSceneSection* Section = MediaTrack->AddNewMediaSourceProxy(
						MediaSource, TargetObjectBindingID, Index, FFrameNumber(0));
					if (Section != nullptr)
					{
						// Tell the section it has a player proxy.
						UMovieSceneMediaSection* MediaSection = Cast<UMovieSceneMediaSection>(Section);
						if (MediaSection != nullptr)
						{
							MediaSection->bHasMediaPlayerProxy = true;
						}
					}
				}
			}
		}
	}

	// Make sure the media plate's player is not playing
	Component->Close();

	// Helper function to implement the "apply to all" logic and response tracking over multiple operations.
	auto AskForDisable = [](const FText& InMessage, FDialogResponseInfo& InResponseInfo)
	{
		// We are using the current transaction id to establish a context for our response info.
		const FGuid CurrentContextId = GUndo ? GUndo->GetContext().TransactionId : FGuid();

		// See if we need to ask again or can reuse previous answer.	
		if (!InResponseInfo.ContextId.IsValid() || InResponseInfo.ContextId != CurrentContextId || !InResponseInfo.bAll)
		{
			const EAppReturnType::Type Reply = FMessageDialog::Open(EAppMsgType::YesNoYesAllNoAll, InMessage);
			InResponseInfo.ContextId = CurrentContextId;
			InResponseInfo.bAll = Reply == EAppReturnType::YesAll || Reply == EAppReturnType::NoAll;
			InResponseInfo.bApply = Reply == EAppReturnType::Yes || Reply == EAppReturnType::YesAll;
		}
		return InResponseInfo.bApply;
	};

	// Does the media plate have autoplay?
	if (Component->bAutoPlay)
	{
		// Ask the user if it can be disabled.
		if (AskForDisable(LOCTEXT("Prompt_DisableAutoplay", "Would you like to disable autoplay on the MediaPlate actor (recommended)?"), LastAutoPlayDisableResponse))
		{
			Component->Modify();
			Component->bAutoPlay = false;
		}
	}

	// Media Track doesn't support audio from the media player yet.
	if (Component->bEnableAudio)
	{
		// Ask the user if it can be disabled.
		if (AskForDisable(LOCTEXT("Prompt_DisableAudio", "Would you like to disable audio on the MediaPlate actor (recommended)?"), LastAudioDisableResponse))
		{
			Component->Modify();
			Component->SetEnableAudio(false);
		}
	}
}

void FMediaPlateTrackEditor::OnRelease()
{
	if (GetSequencer().IsValid())
	{
		if (OnActorAddedToSequencerHandle.IsValid())
		{
			GetSequencer()->OnActorAddedToSequencer().Remove(OnActorAddedToSequencerHandle);
		}
	}

	FMovieSceneTrackEditor::OnRelease();
}

void FMediaPlateTrackEditor::ImportObjectBinding(const TArray<FGuid> ObjectBindings)
{
	// Get actor.
	const TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	for (const FGuid ObjectBinding : ObjectBindings)
	{
		UObject* BoundObject = SequencerPtr.IsValid() ? SequencerPtr->FindSpawnedObjectOrTemplate(ObjectBinding) : nullptr;
		AActor* Actor = Cast<AActor>(BoundObject);
		if (Actor != nullptr)
		{
			// Get media plate componnent.
			if (UMediaPlateComponent* MediaPlateComponent = Actor->FindComponentByClass<UMediaPlateComponent>())
			{
				// Add tracks for this.
				AddTrackForComponent(MediaPlateComponent, ObjectBinding);
			}
		}
	}
}

void FMediaPlateTrackEditor::OnBuildOutlinerEditWidget(FMenuBuilder& MenuBuilder)
{
	// Add media plate sub menu.
	MenuBuilder.AddSubMenu(
		LOCTEXT("MediaPlate", "Media Plate"),
		LOCTEXT("MediaPlateTooltip", "Add media from a media plate."),
		FNewMenuDelegate::CreateLambda([this](FMenuBuilder& MenuBuilder)
		{
			// Actor picker options.
			FSceneOutlinerInitializationOptions InitOptions;
			InitOptions.bShowHeaderRow = false;
			InitOptions.bShowSearchBox = true;
			InitOptions.bShowCreateNewFolder = false;
			InitOptions.bFocusSearchBoxWhenOpened = true;
			InitOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::Label(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 0));
			InitOptions.Filters->AddFilterPredicate<FActorTreeItem>(
				FActorTreeItem::FFilterPredicate::CreateLambda([](const AActor* Actor)
				{
					return Actor && Actor->IsA<AMediaPlate>();
				}
				));

			TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
			const float WidthOverride = SequencerPtr.IsValid() ? SequencerPtr->GetSequencerSettings()->GetAssetBrowserWidth() : 500.f;
			const float HeightOverride = SequencerPtr.IsValid() ? SequencerPtr->GetSequencerSettings()->GetAssetBrowserHeight() : 400.f;

			// Create actor picker.
			FSceneOutlinerModule& SceneOutlinerModule = 
				FModuleManager::LoadModuleChecked<FSceneOutlinerModule>("SceneOutliner");
			TSharedRef<SBox> Picker = SNew(SBox)
				.WidthOverride(WidthOverride)
				.HeightOverride(HeightOverride)
				[
					SceneOutlinerModule.CreateActorPicker(InitOptions,
						FOnActorPicked::CreateLambda([this](AActor* Actor)
						{
							AddMediaPlateToSequencer(Actor);
						}))
				];

			MenuBuilder.AddWidget(Picker, FText::GetEmpty(), true);
		}));
}

void FMediaPlateTrackEditor::AddMediaPlateToSequencer(AActor* Actor)
{
	if (Actor != nullptr)
	{
		TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
		if (SequencerPtr.IsValid())
		{
			TArray<TWeakObjectPtr<AActor>> ActorArray;
			ActorArray.Add(Actor);
			SequencerPtr->AddActors(ActorArray);
		}
	}
}

#undef LOCTEXT_NAMESPACE
