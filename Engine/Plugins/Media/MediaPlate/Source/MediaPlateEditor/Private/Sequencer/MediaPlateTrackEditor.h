// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimatedPropertyKey.h"
#include "Containers/Map.h"
#include "MovieSceneTrackEditor.h"
#include "Templates/SharedPointer.h"

class AActor;
class ISequencer;
class UMediaPlateComponent;

/**
 * Track editor for media plate.
 */
class FMediaPlateTrackEditor
	: public FMovieSceneTrackEditor
{
public:

	/**
	 * Create a new media plate track editor instance.
	 *
	 * @param OwningSequencer The sequencer object that will own the track editor.
	 * @return The new track editor.
	 */
	static TSharedRef<ISequencerTrackEditor>  CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer)
	{
		return MakeShared<FMediaPlateTrackEditor>(OwningSequencer);
	}

	/**
	 * Get the list of all property types that this track editor animates.
	 *
	 * @return List of animated properties.
	 */
	static TArray<FAnimatedPropertyKey, TInlineAllocator<1>> GetAnimatedPropertyTypes();

public:

	/**
	 * Create and initialize a new instance.
	 *
	 * @param InSequencer The sequencer object that owns this track editor.
	 */
	FMediaPlateTrackEditor(TSharedRef<ISequencer> InSequencer);

	/** Virtual destructor. */
	virtual ~FMediaPlateTrackEditor();

public:

	//~ FMovieSceneTrackEditor interface
	virtual FText GetDisplayName() const override;
	virtual void BuildObjectBindingContextMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass) override;
	virtual void BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass) override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneTrack> TrackClass) const override;
	//~ ISequencerTrackEditor interface
	virtual void OnRelease() override;

private:
	/**
	 * Callback for enabling the action to add a media track to an object binding.
	 */
	bool HandleAddMediaTrackToObjectBindingMenuEntryCanExecute(TArray<FGuid> InObjectBindingID);
	/**
	 * Callback for adding a media track to an object binding.
	 */
	void HandleAddMediaTrackToObjectBindingMenuEntryExecute(TArray<FGuid> InObjectBindingID);
	/**
	 * Callback when an actor is added.
	 **/
	void HandleActorAdded(AActor* Actor, FGuid TargetObjectGuid);
	/**
	 * Adds a track for this component.
	 */
	void AddTrackForComponent(UMediaPlateComponent* Component, FGuid TargetObjectGuid);

	/**
	 * Adds tracks from an object binding.
	 */
	void ImportObjectBinding(const TArray<FGuid> ObjectBindings);

	/**
	 * Adds media plates to the MediaTrackEditor widget.
	 * 
	 * @param MenuBuilder		Menu to add to.
	 */
	void OnBuildOutlinerEditWidget(FMenuBuilder& MenuBuilder);

	/**
	 * Adds a media plate to a Sequencer track.
	 * 
	 * @param Actor			Media plate actor to add.
	 */
	void AddMediaPlateToSequencer(AActor* Actor);

	/** Handle to our delegate. */
	FDelegateHandle OnActorAddedToSequencerHandle;

	/**
	 * Information on the last dialog response.
	 */
	struct FDialogResponseInfo
	{
		/** Keep track of the context id for the response to all during multiple operations. */
		FGuid ContextId;
		/** True if the response applies to all in multiple operations under the same context id. */
		bool bAll = false;
		/** True if the operation should be done. */
		bool bApply = false;
	};

	FDialogResponseInfo LastAutoPlayDisableResponse;
	FDialogResponseInfo LastAudioDisableResponse;
};
