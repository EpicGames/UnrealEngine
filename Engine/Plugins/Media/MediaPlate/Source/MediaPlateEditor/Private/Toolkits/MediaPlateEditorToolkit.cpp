// Copyright Epic Games, Inc. All Rights Reserved.

#include "Toolkits/MediaPlateEditorToolkit.h"

#include "Editor.h"
#include "Engine/Engine.h"
#include "EditorReimportHandler.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IMediaAssetsModule.h"
#include "MediaPlate.h"
#include "MediaPlateComponent.h"
#include "MediaPlateCustomization.h"
#include "MediaPlateEditorModule.h"
#include "MediaPlayer.h"
#include "MediaPlaylist.h"
#include "Models/MediaPlayerEditorCommands.h"
#include "SlateOptMacros.h"
#include "Styling/AppStyle.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SMediaImageTextureChannelToggle.h"
#include "Widgets/SMediaPlateEditorDetails.h"
#include "Widgets/SMediaPlateEditorMediaDetails.h"
#include "Widgets/SMediaPlateEditorPlaylist.h"
#include "Widgets/SMediaPlayerEditorViewer.h"

#define LOCTEXT_NAMESPACE "FMediaPlateEditorToolkit"

namespace MediaPlateEditorToolkit
{
	static const FName AppIdentifier("MediaPlateEditorApp");
	static const FName DetailsTabId("Details");
	static const FName MediaDetailsTabId("MediaDetails");
	static const FName PlaylistTabId("Playlist");
	static const FName ViewerTabId("Viewer");
}

/* FMediaPlateEditorToolkit structors
 *****************************************************************************/

FMediaPlateEditorToolkit::FMediaPlateEditorToolkit(const TSharedRef<ISlateStyle>& InStyle)
	: MediaPlate(nullptr)
	, Style(InStyle)
{
}

FMediaPlateEditorToolkit::~FMediaPlateEditorToolkit()
{
	FReimportManager::Instance()->OnPreReimport().RemoveAll(this);
	FReimportManager::Instance()->OnPostReimport().RemoveAll(this);

	if (GEngine)
	{
		GEngine->OnLevelActorDeleted().RemoveAll(this);
	}

	if (GEditor)
	{
		GEditor->UnregisterForUndo(this);
	}
}

/* FMediaPlateEditorToolkit interface
 *****************************************************************************/

void FMediaPlateEditorToolkit::Initialize(UMediaPlateComponent* InMediaPlate, const EToolkitMode::Type InMode, const TSharedPtr<IToolkitHost>& InToolkitHost)
{
	MediaPlate = InMediaPlate;

	if (MediaPlate == nullptr)
	{
		return;
	}

	// support undo/redo
	MediaPlate->SetFlags(RF_Transactional);

	if (GEditor)
	{
		GEditor->RegisterForUndo(this);
	}

	BindCommands();

	// create tab layout
	const TSharedRef<FTabManager::FLayout> Layout = FTabManager::NewLayout("Standalone_MediaPlateEditor_v1.3")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
				->SetOrientation(Orient_Vertical)
				->Split
				(
					FTabManager::NewSplitter()
						->SetOrientation(Orient_Horizontal)
						->SetSizeCoefficient(0.7f)
						->Split
						(
							// viewer
							FTabManager::NewStack()
								->AddTab(MediaPlateEditorToolkit::ViewerTabId, ETabState::OpenedTab)
								->SetHideTabWell(true)
								->SetSizeCoefficient(0.6f)
						)	
						->Split
						(
							FTabManager::NewSplitter()
							->SetOrientation(Orient_Vertical)
							->SetSizeCoefficient(0.4f)
							->Split
							(
								// Media details tab.
								FTabManager::NewStack()
								->AddTab(MediaPlateEditorToolkit::MediaDetailsTabId, ETabState::OpenedTab)
								->SetSizeCoefficient(0.2f)
							)
							->Split
							(
								// Details tab.
								FTabManager::NewStack()
								->AddTab(MediaPlateEditorToolkit::DetailsTabId, ETabState::OpenedTab)
								->SetSizeCoefficient(0.8f)
							)
						)
				)
				->Split
				(
					// Details tab.
					FTabManager::NewStack()
						->AddTab(MediaPlateEditorToolkit::PlaylistTabId, ETabState::OpenedTab)
						->SetSizeCoefficient(0.3f)
				)
		);

	FAssetEditorToolkit::InitAssetEditor(
		InMode,
		InToolkitHost,
		MediaPlateEditorToolkit::AppIdentifier,
		Layout,
		true,
		true,
		InMediaPlate
	);
	
	ExtendToolBar();
	RegenerateMenusAndToolbars();

	// Tell the editor module that this media plate is playing.
	FMediaPlateEditorModule* EditorModule = FModuleManager::LoadModulePtr<FMediaPlateEditorModule>("MediaPlateEditor");
	if (EditorModule != nullptr)
	{
		EditorModule->MediaPlateStartedPlayback(MediaPlate);
	}

	if (GEngine)
	{
		GEngine->OnLevelActorDeleted().AddSP(this, &FMediaPlateEditorToolkit::OnActorDeleted);
	}
}

/* FAssetEditorToolkit interface
 *****************************************************************************/

void FMediaPlateEditorToolkit::ToggleChannelMask(MediaPlayerEditor::MediaImage::ETextureChannelMask InChannelToToggle)
{
	if (!Viewer.IsValid())
	{
		return;
	}

	using namespace MediaPlayerEditor::MediaImage;

	const ETextureChannelMask CurrentMask = Viewer->GetChannelMask();

	if (EnumHasAllFlags(CurrentMask, InChannelToToggle))
	{
		Viewer->SetChannelMask(CurrentMask & ~InChannelToToggle);
	}
	else
	{
		Viewer->SetChannelMask(CurrentMask | InChannelToToggle);
	}
}

FString FMediaPlateEditorToolkit::GetDocumentationLink() const
{
	return FString(TEXT("WorkingWithMedia/IntegratingMedia/MediaFramework"));
}

void FMediaPlateEditorToolkit::OnClose()
{
	HandleMediaPlateEvent(EMediaPlateEventState::Close);
}

void FMediaPlateEditorToolkit::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_MediaPlateEditor", "Media Plate Editor"));
	auto WorkspaceMenuCategoryRef = WorkspaceMenuCategory.ToSharedRef();

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	// Details tab.
	InTabManager->RegisterTabSpawner(MediaPlateEditorToolkit::DetailsTabId, FOnSpawnTab::CreateSP(this, &FMediaPlateEditorToolkit::HandleTabManagerSpawnTab, MediaPlateEditorToolkit::DetailsTabId))
		.SetDisplayName(LOCTEXT("DetailsTabName", "Details"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	// Media details tab.
	InTabManager->RegisterTabSpawner(MediaPlateEditorToolkit::MediaDetailsTabId, FOnSpawnTab::CreateSP(this, &FMediaPlateEditorToolkit::HandleTabManagerSpawnTab, MediaPlateEditorToolkit::MediaDetailsTabId))
		.SetDisplayName(LOCTEXT("MediaDetailsTabName", "Media Details"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	// Playlist tab.
	InTabManager->RegisterTabSpawner(MediaPlateEditorToolkit::PlaylistTabId, FOnSpawnTab::CreateSP(this, &FMediaPlateEditorToolkit::HandleTabManagerSpawnTab, MediaPlateEditorToolkit::PlaylistTabId))
		.SetDisplayName(LOCTEXT("PlaylistTabName", "Playlist"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	// Viewer tab.
	InTabManager->RegisterTabSpawner(MediaPlateEditorToolkit::ViewerTabId, FOnSpawnTab::CreateSP(this, &FMediaPlateEditorToolkit::HandleTabManagerSpawnTab, MediaPlateEditorToolkit::ViewerTabId))
		.SetDisplayName(LOCTEXT("PlayerTabName", "Player"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(Style->GetStyleSetName(), "MediaPlateEditor.Tabs.Player"));
}

void FMediaPlateEditorToolkit::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner(MediaPlateEditorToolkit::ViewerTabId);
	InTabManager->UnregisterTabSpawner(MediaPlateEditorToolkit::PlaylistTabId);
	InTabManager->UnregisterTabSpawner(MediaPlateEditorToolkit::MediaDetailsTabId);
	InTabManager->UnregisterTabSpawner(MediaPlateEditorToolkit::DetailsTabId);
}

/* IToolkit interface
 *****************************************************************************/

FText FMediaPlateEditorToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Media Plate Editor");
}

FName FMediaPlateEditorToolkit::GetToolkitFName() const
{
	return FName("MediaPlateEditor");
}

FLinearColor FMediaPlateEditorToolkit::GetWorldCentricTabColorScale() const
{
	return FLinearColor(0.3f, 0.2f, 0.5f, 0.5f);
}

FString FMediaPlateEditorToolkit::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "MediaPlate ").ToString();
}

/* FGCObject interface
 *****************************************************************************/

void FMediaPlateEditorToolkit::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(MediaPlate);
}

/* FEditorUndoClient interface
*****************************************************************************/

void FMediaPlateEditorToolkit::PostUndo(bool bSuccess)
{
	// do nothing
}

void FMediaPlateEditorToolkit::PostRedo(bool bSuccess)
{
	PostUndo(bSuccess);
}

/* FMediaPlayerEditorToolkit implementation
 *****************************************************************************/

void FMediaPlateEditorToolkit::BindCommands()
{
	const FMediaPlayerEditorCommands& Commands = FMediaPlayerEditorCommands::GetExternal();

	ToolkitCommands->MapAction(
		Commands.CloseMedia,
		FExecuteAction::CreateSP( this, &FMediaPlateEditorToolkit::HandleMediaPlateEvent, EMediaPlateEventState::Close ),
		FCanExecuteAction::CreateSP( this, &FMediaPlateEditorToolkit::CanHandleMediaPlateEvent, EMediaPlateEventState::Close )
	);

	ToolkitCommands->MapAction(
		Commands.ForwardMedia,
		FExecuteAction::CreateSP( this, &FMediaPlateEditorToolkit::HandleMediaPlateEvent, EMediaPlateEventState::Forward ),
		FCanExecuteAction::CreateSP( this, &FMediaPlateEditorToolkit::CanHandleMediaPlateEvent, EMediaPlateEventState::Forward )
	);

	ToolkitCommands->MapAction(
		Commands.NextMedia,
		FExecuteAction::CreateSP( this, &FMediaPlateEditorToolkit::HandleMediaPlateEvent, EMediaPlateEventState::Next ),
		FCanExecuteAction::CreateSP( this, &FMediaPlateEditorToolkit::CanHandleMediaPlateEvent, EMediaPlateEventState::Next )
	);

	ToolkitCommands->MapAction(
		Commands.OpenMedia,
		FExecuteAction::CreateSP( this, &FMediaPlateEditorToolkit::HandleMediaPlateEvent, EMediaPlateEventState::Open ),
		FCanExecuteAction::CreateSP( this, &FMediaPlateEditorToolkit::CanHandleMediaPlateEvent, EMediaPlateEventState::Open )
	);

	ToolkitCommands->MapAction(
		Commands.PauseMedia,
		FExecuteAction::CreateSP( this, &FMediaPlateEditorToolkit::HandleMediaPlateEvent, EMediaPlateEventState::Pause ),
		FCanExecuteAction::CreateSP( this, &FMediaPlateEditorToolkit::CanHandleMediaPlateEvent, EMediaPlateEventState::Pause )
	);

	ToolkitCommands->MapAction(
		Commands.PlayMedia,
		FExecuteAction::CreateSP( this, &FMediaPlateEditorToolkit::HandleMediaPlateEvent, EMediaPlateEventState::Play ),
		FCanExecuteAction::CreateSP( this, &FMediaPlateEditorToolkit::CanHandleMediaPlateEvent, EMediaPlateEventState::Play )
	);

	ToolkitCommands->MapAction(
		Commands.PlayReverseMedia,
		FExecuteAction::CreateSP(this, &FMediaPlateEditorToolkit::HandleMediaPlateEvent, EMediaPlateEventState::PlayReverse),
		FCanExecuteAction::CreateSP(this, &FMediaPlateEditorToolkit::CanHandleMediaPlateEvent, EMediaPlateEventState::PlayReverse)
	);

	ToolkitCommands->MapAction(
		Commands.PreviousMedia,
		FExecuteAction::CreateSP( this, &FMediaPlateEditorToolkit::HandleMediaPlateEvent, EMediaPlateEventState::Previous ),
		FCanExecuteAction::CreateSP( this, &FMediaPlateEditorToolkit::CanHandleMediaPlateEvent, EMediaPlateEventState::Previous )
	);

	ToolkitCommands->MapAction(
		Commands.ReverseMedia,
		FExecuteAction::CreateSP( this, &FMediaPlateEditorToolkit::HandleMediaPlateEvent, EMediaPlateEventState::Reverse ),
		FCanExecuteAction::CreateSP( this, &FMediaPlateEditorToolkit::CanHandleMediaPlateEvent, EMediaPlateEventState::Reverse )
	);

	ToolkitCommands->MapAction(
		Commands.RewindMedia,
		FExecuteAction::CreateSP(this, &FMediaPlateEditorToolkit::HandleMediaPlateEvent, EMediaPlateEventState::Rewind),
		FCanExecuteAction::CreateSP(this, &FMediaPlateEditorToolkit::CanHandleMediaPlateEvent, EMediaPlateEventState::Rewind)
	);

	ToolkitCommands->MapAction(
		Commands.JumpToEndMedia,
		FExecuteAction::CreateSP(this, &FMediaPlateEditorToolkit::HandleMediaPlateEvent, EMediaPlateEventState::JumpToEnd),
		FCanExecuteAction::CreateSP(this, &FMediaPlateEditorToolkit::CanHandleMediaPlateEvent, EMediaPlateEventState::JumpToEnd)
	);

	ToolkitCommands->MapAction(
		Commands.TogglePlayPauseMedia,
		FExecuteAction::CreateSP( this, &FMediaPlateEditorToolkit::HandleMediaPlateTogglePlayPauseEvent ),
		FCanExecuteAction::CreateSP( this, &FMediaPlateEditorToolkit::CanHandleMediaPlateTogglePlayPauseEvent )
	);

	ToolkitCommands->MapAction(
		Commands.TogglePlayReversePauseMedia,
		FExecuteAction::CreateSP(this, &FMediaPlateEditorToolkit::HandleMediaPlateTogglePlayReversePauseEvent),
		FCanExecuteAction::CreateSP(this, &FMediaPlateEditorToolkit::CanHandleMediaPlateTogglePlayReversePauseEvent)
	);

	ToolkitCommands->MapAction(
		Commands.StepForwardMedia,
		FExecuteAction::CreateSP(this, &FMediaPlateEditorToolkit::HandleMediaPlateEvent, EMediaPlateEventState::StepForward),
		FCanExecuteAction::CreateSP(this, &FMediaPlateEditorToolkit::CanHandleMediaPlateEvent, EMediaPlateEventState::StepForward)
	);

	ToolkitCommands->MapAction(
		Commands.StepBackwardMedia,
		FExecuteAction::CreateSP(this, &FMediaPlateEditorToolkit::HandleMediaPlateEvent, EMediaPlateEventState::StepBackward),
		FCanExecuteAction::CreateSP(this, &FMediaPlateEditorToolkit::CanHandleMediaPlateEvent, EMediaPlateEventState::StepBackward)
	);

	ToolkitCommands->MapAction(
		Commands.ToggledRedTextureChannel,
		FExecuteAction::CreateLambda([this] { ToggleChannelMask(MediaPlayerEditor::MediaImage::ETextureChannelMask::Red); }),
		FCanExecuteAction::CreateLambda([this] { return Viewer.IsValid(); }),
		FIsActionChecked::CreateLambda([this] { return IsChannelMasked(MediaPlayerEditor::MediaImage::ETextureChannelMask::Red); })
	);

	ToolkitCommands->MapAction(
		Commands.ToggledGreenTextureChannel,
		FExecuteAction::CreateLambda([this] { ToggleChannelMask(MediaPlayerEditor::MediaImage::ETextureChannelMask::Green); }),
		FCanExecuteAction::CreateLambda([this] { return Viewer.IsValid(); }),
		FIsActionChecked::CreateLambda([this] { return IsChannelMasked(MediaPlayerEditor::MediaImage::ETextureChannelMask::Green); })
	);

	ToolkitCommands->MapAction(
		Commands.ToggledBlueTextureChannel,
		FExecuteAction::CreateLambda([this] { ToggleChannelMask(MediaPlayerEditor::MediaImage::ETextureChannelMask::Blue); }),
		FCanExecuteAction::CreateLambda([this] { return Viewer.IsValid(); }),
		FIsActionChecked::CreateLambda([this] { return IsChannelMasked(MediaPlayerEditor::MediaImage::ETextureChannelMask::Blue); })
	);

	ToolkitCommands->MapAction(
		Commands.ToggledAlphaTextureChannel,
		FExecuteAction::CreateLambda([this] { ToggleChannelMask(MediaPlayerEditor::MediaImage::ETextureChannelMask::Alpha); }),
		FCanExecuteAction::CreateLambda([this] { return Viewer.IsValid(); }),
		FIsActionChecked::CreateLambda([this] { return IsChannelMasked(MediaPlayerEditor::MediaImage::ETextureChannelMask::Alpha); })
	);
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void FMediaPlateEditorToolkit::ExtendToolBar()
{
	TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);

	ToolbarExtender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateLambda(
			[this](FToolBarBuilder& ToolbarBuilder)
			{
				using namespace MediaPlayerEditor::MediaImage;

				ToolbarBuilder.BeginSection("TextureControls");
				{
					ToolbarBuilder.AddWidget(SNew(SMediaImageTextureChannelToggle, Viewer, ETextureChannelMask::Red,   ToolkitCommands));
					ToolbarBuilder.AddWidget(SNew(SMediaImageTextureChannelToggle, Viewer, ETextureChannelMask::Green, ToolkitCommands));
					ToolbarBuilder.AddWidget(SNew(SMediaImageTextureChannelToggle, Viewer, ETextureChannelMask::Blue,  ToolkitCommands));
					ToolbarBuilder.AddWidget(SNew(SMediaImageTextureChannelToggle, Viewer, ETextureChannelMask::Alpha, ToolkitCommands));
				}
				ToolbarBuilder.EndSection();
			}
		)
	);

	AddToolbarExtender(ToolbarExtender);
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

float FMediaPlateEditorToolkit::GetForwardRate() const
{
	TObjectPtr<UMediaPlayer> MediaPlayer = MediaPlate->GetMediaPlayer();
	float Rate = MediaPlayer->GetRate();

	if (Rate < 1.0f)
	{
		Rate = 1.0f;
	}

	return 2.0f * Rate;
}

float FMediaPlateEditorToolkit::GetReverseRate() const
{
	TObjectPtr<UMediaPlayer> MediaPlayer = MediaPlate->GetMediaPlayer();
	float Rate = MediaPlayer->GetRate();

	if (Rate > -1.0f)
	{
		return -1.0f;
	}

	return 2.0f * Rate;
}

/* FMediaPlayerEditorToolkit callbacks
 *****************************************************************************/

bool FMediaPlateEditorToolkit::CanHandleMediaPlateEvent(EMediaPlateEventState InState)
{
	if (MediaPlate)
	{
		switch (InState)
		{
			case EMediaPlateEventState::Next:
				return (MediaPlate->GetMediaPlaylist() != nullptr)
					&& (MediaPlate->GetMediaPlaylist()->Num() > 1);
			case EMediaPlateEventState::Previous:
				return (MediaPlate->GetMediaPlaylist() != nullptr)
					&& (MediaPlate->GetMediaPlaylist()->Num() > 1)
					&& (MediaPlate->PlaylistIndex > 0); // Match UMediaPlateComponent::Previous() conditions.
		default:
			return FMediaPlateCustomization::IsMediaPlateEventAllowedForPlayer(InState, MediaPlate->GetMediaPlayer());	
		}
	}
	return false;
}

void FMediaPlateEditorToolkit::HandleMediaPlateEvent(EMediaPlateEventState InState)
{
	const TArray<TWeakObjectPtr<UMediaPlateComponent>, TInlineAllocator<1>> MediaPlates = {MediaPlate};
	FMediaPlateCustomization::HandleMediaPlateEvent(MediaPlates, InState);
}

bool FMediaPlateEditorToolkit::CanHandleMediaPlateTogglePlayPauseEvent()
{
	if (UMediaPlayer* MediaPlayer = MediaPlate->GetMediaPlayer())
	{
		if (MediaPlayer->IsReady())
		{
			if (MediaPlayer->IsPlaying() && MediaPlayer->GetRate() > 0)
			{
				return CanHandleMediaPlateEvent(EMediaPlateEventState::Pause);
			}
			else
			{
				return CanHandleMediaPlateEvent(EMediaPlateEventState::Play);
			}
		}
	}

	return false;
}

void FMediaPlateEditorToolkit::HandleMediaPlateTogglePlayPauseEvent()
{
	if (UMediaPlayer* MediaPlayer = MediaPlate->GetMediaPlayer())
	{
		if (MediaPlayer->IsReady())
		{
			if (MediaPlayer->IsPlaying() && MediaPlayer->GetRate() > 0)
			{
				HandleMediaPlateEvent(EMediaPlateEventState::Pause);
			}
			else
			{
				HandleMediaPlateEvent(EMediaPlateEventState::Play);
			}
		}
	}
}

bool FMediaPlateEditorToolkit::CanHandleMediaPlateTogglePlayReversePauseEvent()
{
	if (UMediaPlayer* MediaPlayer = MediaPlate->GetMediaPlayer())
	{
		if (MediaPlayer->IsReady() && MediaPlayer->SupportsRate(-1.f, /* Unthinned */ true))
		{
			if (MediaPlayer->IsPlaying() && MediaPlayer->GetRate() < 0)
			{
				return CanHandleMediaPlateEvent(EMediaPlateEventState::Pause);
			}
			else
			{
				return CanHandleMediaPlateEvent(EMediaPlateEventState::PlayReverse);
			}
		}
	}

	return false;
}

void FMediaPlateEditorToolkit::HandleMediaPlateTogglePlayReversePauseEvent()
{
	if (UMediaPlayer* MediaPlayer = MediaPlate->GetMediaPlayer())
	{
		if (MediaPlayer->IsReady() && MediaPlayer->SupportsRate(-1.f, /* Unthinned */ true))
		{
			if (MediaPlayer->IsPlaying() && MediaPlayer->GetRate() < 0)
			{
				HandleMediaPlateEvent(EMediaPlateEventState::Pause);
			}
			else
			{
				HandleMediaPlateEvent(EMediaPlateEventState::PlayReverse);
			}
		}
	}
}

TSharedRef<SDockTab> FMediaPlateEditorToolkit::HandleTabManagerSpawnTab(const FSpawnTabArgs& Args, FName TabIdentifier)
{
	UMediaPlayer* MediaPlayer = MediaPlate->GetMediaPlayer();
	UMediaTexture* MediaTexture = MediaPlate->GetMediaTexture();
	TSharedPtr<SWidget> TabWidget = SNullWidget::NullWidget;

	if (TabIdentifier == MediaPlateEditorToolkit::DetailsTabId)
	{
		TabWidget = SNew(SMediaPlateEditorDetails, *MediaPlate, Style);
	}
	else if (TabIdentifier == MediaPlateEditorToolkit::MediaDetailsTabId)
	{
		TabWidget = SNew(SMediaPlateEditorMediaDetails, *MediaPlate);
	}
	else if (TabIdentifier == MediaPlateEditorToolkit::PlaylistTabId)
	{
		TabWidget = SNew(SMediaPlateEditorPlaylist, *MediaPlate, Style);
	}
	else if (TabIdentifier == MediaPlateEditorToolkit::ViewerTabId)
	{
		if (MediaPlayer != nullptr)
		{
			TabWidget = SAssignNew(Viewer, SMediaPlayerEditorViewer, *MediaPlayer, MediaTexture, Style, false)
				.Commands(ToolkitCommands);
		}
	}

	return SNew(SDockTab)
		.TabRole(ETabRole::PanelTab)
		[
			TabWidget.ToSharedRef()
		];
}

void FMediaPlateEditorToolkit::OnActorDeleted(AActor* Actor)
{
	// If our actor owning actor is removed, we need to close. Our MediaPlate referenced will be cleared by GC and cause a crash later.
	if (MediaPlate && MediaPlate->GetOwner() == Actor)
	{
		CloseWindow(EAssetEditorCloseReason::AssetUnloadingOrInvalid);
	}
}

bool FMediaPlateEditorToolkit::IsChannelMasked(MediaPlayerEditor::MediaImage::ETextureChannelMask InChannel) const
{
	if (Viewer.IsValid())
	{
		return EnumHasAllFlags(Viewer->GetChannelMask(), InChannel);
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
