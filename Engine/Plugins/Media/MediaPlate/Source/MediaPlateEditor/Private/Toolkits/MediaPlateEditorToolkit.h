// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorUndoClient.h"
#include "Framework/Docking/TabManager.h"
#include "Styling/ISlateStyle.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "UObject/GCObject.h"
#include "Widgets/SMediaImage.h"

class SMediaPlayerEditorViewer;
class UMediaPlateComponent;
enum class EMediaPlateEventState : uint8;

/**
 * Implements an Editor toolkit for media plates.
 */
class FMediaPlateEditorToolkit
	: public FAssetEditorToolkit
	, public FEditorUndoClient
	, public FGCObject
{
public:
	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InStyle The style set to use.
	 */
	FMediaPlateEditorToolkit(const TSharedRef<ISlateStyle>& InStyle);

	/** Virtual destructor. */
	virtual ~FMediaPlateEditorToolkit();
	/**
	 * Initializes the editor tool kit.
	 *
	 * @param InMediaPlate The media plate asset to edit.
	 * @param InMode The mode to create the toolkit in.
	 * @param InToolkitHost The toolkit host.
	 */
	void Initialize(UMediaPlateComponent* InMediaPlate, const EToolkitMode::Type InMode, const TSharedPtr<IToolkitHost>& InToolkitHost);

	/**
	 * Returns true if all the given channel mask is enabled.
	 */
	bool IsChannelMasked(MediaPlayerEditor::MediaImage::ETextureChannelMask InChannel) const;

	/**
	 * Toggles the given mask in the channel mask.
	 */
	void ToggleChannelMask(MediaPlayerEditor::MediaImage::ETextureChannelMask InChannelToToggle);

	//~ FAssetEditorToolkit interface
	virtual FString GetDocumentationLink() const override;
	virtual void OnClose() override;
	virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;

	//~ IToolkit interface
	virtual FText GetBaseToolkitName() const override;
	virtual FName GetToolkitFName() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	virtual FString GetWorldCentricTabPrefix() const override;

	//~ FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FMediaPlateEditorToolkit");
	}

protected:

	//~ FEditorUndoClient interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;

	/** Binds the UI commands to delegates. */
	void BindCommands();

	/** Builds the toolbar widget for the media player editor. */
	void ExtendToolBar();

	/**
	 * Gets the playback rate for fast forward.
	 *
	 * @return Forward playback rate.
	 */
	float GetForwardRate() const;

	/**
	 * Gets the playback rate for reverse.
	 *
	 * @return Reverse playback rate.
	 */
	float GetReverseRate() const;

private:
	/** The media plate asset being edited. */
	TObjectPtr<UMediaPlateComponent> MediaPlate;

	/** Pointer to the style set to use for toolkits. */
	TSharedRef<ISlateStyle> Style;

	/** The tab which displays the media. */
	TSharedPtr<SMediaPlayerEditorViewer> Viewer;

	/** Callback to validate playback button events. */
	bool CanHandleMediaPlateEvent(EMediaPlateEventState InState);

	/** Callback to handle playback button events. */
	void HandleMediaPlateEvent(EMediaPlateEventState InState);
	
	/** Callback specifically to validate the toggle between play and pause. */
	bool CanHandleMediaPlateTogglePlayPauseEvent();

	/** Callback specifically for toggle play/pause which then triggers a play or pause event. */
	void HandleMediaPlateTogglePlayPauseEvent();

	/** Callback specifically to validate the toggle between reverse play and pause. */
	bool CanHandleMediaPlateTogglePlayReversePauseEvent();

	/** Callback specifically for toggle play/pause which then triggers a reverse play or pause event. */
	void HandleMediaPlateTogglePlayReversePauseEvent();

	/** Callback for spawning tabs. */
	TSharedRef<SDockTab> HandleTabManagerSpawnTab(const FSpawnTabArgs& Args, FName TabIdentifier);
	
	void OnActorDeleted(AActor* Actor);
};
