// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMediaPlateEditorDetails.h"
#include "IDetailsView.h"
#include "MediaPlateComponent.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"

#define LOCTEXT_NAMESPACE "SMediaPlateEditorDetails"

/* SMediaPlateEditorDetails interface
 *****************************************************************************/

void SMediaPlateEditorDetails::Construct(const FArguments& InArgs, UMediaPlateComponent& InMediaPlate, const TSharedRef<ISlateStyle>& InStyle)
{
	MediaPlate = &InMediaPlate;

	// Initialize details view
	FDetailsViewArgs DetailsViewArgs;
	{
		DetailsViewArgs.bAllowSearch = true;
		DetailsViewArgs.bHideSelectionTip = true;
		DetailsViewArgs.bLockable = false;
		DetailsViewArgs.bSearchInitialKeyFocus = true;
		DetailsViewArgs.bUpdatesFromSelection = false;
		DetailsViewArgs.bShowOptions = true;
		DetailsViewArgs.bShowModifiedPropertiesOption = false;
	}

	TSharedPtr<IDetailsView> DetailsView = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor").CreateDetailView(DetailsViewArgs);
	DetailsView->SetObject(MediaPlate);

	ChildSlot
	[
		DetailsView.ToSharedRef()
	];
}


#undef LOCTEXT_NAMESPACE
