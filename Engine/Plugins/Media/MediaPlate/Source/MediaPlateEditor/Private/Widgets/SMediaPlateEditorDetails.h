// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/ISlateStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class UMediaPlateComponent;

/**
 * Implements the details panel of the MediaPlate asset editor.
 */
class SMediaPlateEditorDetails
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMediaPlateEditorDetails) { }
	SLATE_END_ARGS()

public:

	/**
	 * Construct this widget.
	 *
	 * @param InArgs		The declaration data for this widget.
	 * @param InMediaPlate	The MediaPlate to show the details for.
	 * @param InStyleSet	The style set to use.
	 */
	void Construct(const FArguments& InArgs, UMediaPlateComponent& InMediaPlate, const TSharedRef<ISlateStyle>& InStyle);

private:

	/** Pointer to the MediaPlate that is being viewed. */
	TObjectPtr<UMediaPlateComponent> MediaPlate;
};
