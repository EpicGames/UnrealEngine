// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMediaPlateEditorMaterial.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Editor.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"
#include "MediaPlate.h"
#include "MediaPlateEditorModule.h"
#include "ScopedTransaction.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"

#define LOCTEXT_NAMESPACE "SMediaPlateEditorMaterial"

/* SMediaPlateEditorMaterial interface
 *****************************************************************************/

void SMediaPlateEditorMaterial::Construct(const FArguments& InArgs, const TSharedRef<FMaterialItemView>& InMaterialItemView, UActorComponent* InCurrentComponent)
{
	TSharedRef<SVerticalBox> ResultWidget = SNew(SVerticalBox);

	// Get media plate.
	if (InCurrentComponent != nullptr)
	{
		MediaPlate = InCurrentComponent->GetOwner<AMediaPlate>();
		if (MediaPlate != nullptr)
		{
			// Add button to browse the material.
			ResultWidget->AddSlot()
				.AutoHeight()
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
						.Text(LOCTEXT("BrowseMaterial", "Browse To Material"))
						.ToolTipText(LOCTEXT("BrowsetMaterialTooltip", "Browse to the material asset in the Content Browser."))
						.OnClicked(this, &SMediaPlateEditorMaterial::OnBrowseMaterial)
				];

			// Add button for default material.
			ResultWidget->AddSlot()
				.AutoHeight()
				.VAlign(VAlign_Center)
				[
					SNew(SComboButton)
						.OnGetMenuContent(this, &SMediaPlateEditorMaterial::OnGetMaterials)
						.ContentPadding(2.0f)
						.ButtonContent()
						[
							SNew(STextBlock)
								.Text(LOCTEXT("SelectMaterialButton", "Select Media Plate Material"))
								.ToolTipText(LOCTEXT("SelectMaterialTooltip", "Select a material to use from the recommended Media Plate materials."))
						]
				];
		}
	}

	ChildSlot
	[
		ResultWidget
	];
}

FReply SMediaPlateEditorMaterial::OnBrowseMaterial() const
{
	if (MediaPlate != nullptr)
	{
		UMaterialInterface* Material = MediaPlate->GetCurrentMaterial();
		if (Material != nullptr)
		{
			UMaterial* BaseMaterial = Material->GetMaterial();
			if (BaseMaterial != nullptr)
			{
				TArray<UObject*> ObjectsToSync;
				ObjectsToSync.Add(BaseMaterial);

				if (GEditor)
				{
					GEditor->SyncBrowserToObjects(ObjectsToSync);
				}
			}
		}
	}

	return FReply::Handled();
}

TSharedRef<SWidget> SMediaPlateEditorMaterial::OnGetMaterials()
{
	FMenuBuilder MenuBuilder(true, NULL);

	// Get media plate assets.
	const IAssetRegistry& AssetRegistry = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();

	TArray<FName> AssetPaths;

	if (FMediaPlateEditorModule* EditorModule = FModuleManager::LoadModulePtr<FMediaPlateEditorModule>("MediaPlateEditor"))
	{
		EditorModule->OnGetMediaPlateMaterialAssetPaths().Broadcast(AssetPaths);
	}

	// Add MediaPlate plugin content path.
	AssetPaths.AddUnique(FName("/MediaPlate"));

	TArray<FAssetData> Assets;
	AssetRegistry.GetAssetsByPaths(AssetPaths, Assets, /*bRecursive*/ true);

	// Sort by name to have a consistent list.
	Algo::Sort(Assets, [](const FAssetData& InAssetA, const FAssetData& InAssetB)
		{ return InAssetA.AssetName.LexicalLess(InAssetB.AssetName); });

	for (const FAssetData& AssetData : Assets)
	{
		if (AssetData.IsInstanceOf(UMaterial::StaticClass()))
		{
			FUIAction Action = FUIAction(FExecuteAction::CreateSP(this, &SMediaPlateEditorMaterial::OnSelectMaterial, AssetData));
			MenuBuilder.AddMenuEntry(FText::FromName(AssetData.AssetName), FText(), FSlateIcon(), Action);
		}
	}

	return MenuBuilder.MakeWidget();
}

void SMediaPlateEditorMaterial::OnSelectMaterial(FAssetData AssetData) const
{
	if (MediaPlate != nullptr)
	{
		UObject* AssetObject = AssetData.GetAsset();
		if (AssetObject != nullptr)
		{
			UMaterial* Material = Cast<UMaterial>(AssetObject);
			if (Material != nullptr)
			{
				const FScopedTransaction Transaction(LOCTEXT("SelectMaterialTransaction", "Media Plate Select Material"));
				MediaPlate->ApplyMaterial(Material);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
