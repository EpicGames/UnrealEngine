// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MaterialList.h"
#include "Widgets/SCompoundWidget.h"

class AMediaPlate;

/**
 * Widget for editing materials with media plate.
 */
class SMediaPlateEditorMaterial : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMediaPlateEditorMaterial) { }
	SLATE_END_ARGS()

public:

	/**
	 * Construct this widget.
	 *
	 * @param InArgs				The declaration data for this widget.
	 * @param InMaterialItemView	Material view.
	 * @param InCurrentComponent	component that is using the material.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<FMaterialItemView>& InMaterialItemView, UActorComponent* InCurrentComponent);

private:
	/**
	 * Callback for when we browse a material.
	 */
	FReply OnBrowseMaterial() const;

	/**
	 * Callback to get a list of media plate materials.
	 */
	TSharedRef<SWidget> OnGetMaterials();

	/**
	 * Callback for when we select a material.
	 */
	void OnSelectMaterial(FAssetData AssetData) const;

	/** Weak pointer to current component using the current material */
	TWeakObjectPtr<AMediaPlate> MediaPlate = nullptr;
};
