// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMediaPlateEditorMediaDetails.h"
#include "MediaPlateComponent.h"
#include "MediaPlayer.h"
#include "MediaTexture.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Layout/SScrollBox.h"

#define LOCTEXT_NAMESPACE "SMediaPlateEditorMediaDetails"

/* SMediaPlateEditorMediaDetails interface
 *****************************************************************************/

void SMediaPlateEditorMediaDetails::Construct(const FArguments& InArgs,
	UMediaPlateComponent& InMediaPlate)
{
	MediaPlate = &InMediaPlate;

	SMediaPlayerEditorMediaDetails::Construct(SMediaPlayerEditorMediaDetails::FArguments(), nullptr, nullptr, TEXT("SmallText"));
}

UMediaPlayer* SMediaPlateEditorMediaDetails::GetMediaPlayer() const
{
	if (UMediaPlateComponent* MediaPlateComponent = MediaPlate.Get())
	{
		return MediaPlateComponent->GetMediaPlayer();
	}
	return nullptr;
}

UMediaTexture* SMediaPlateEditorMediaDetails::GetMediaTexture() const
{
	if (UMediaPlateComponent* MediaPlateComponent = MediaPlate.Get())
	{
		return MediaPlateComponent->GetMediaTexture();
	}
	return nullptr;
}

#undef LOCTEXT_NAMESPACE
