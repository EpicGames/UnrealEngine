// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/ISlateStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SMediaPlayerEditorMediaDetails.h"

class STextBlock;
class UMediaPlateComponent;

/**
 * Implements the details panel of the MediaPlate asset editor.
 */
class SMediaPlateEditorMediaDetails
	: public SMediaPlayerEditorMediaDetails
{
public:

	SLATE_BEGIN_ARGS(SMediaPlateEditorMediaDetails) { }
	SLATE_END_ARGS()

public:

	/**
	 * Construct this widget.
	 *
	 * @param InArgs		The declaration data for this widget.
	 * @param InMediaPlate	The MediaPlate to show the details for.
	 */
	void Construct(const FArguments& InArgs, UMediaPlateComponent& InMediaPlate);

protected:
	//~ Begin SMediaPlayerEditorMediaDetails
	virtual UMediaPlayer* GetMediaPlayer() const override;
	virtual UMediaTexture* GetMediaTexture() const override;
	//~ End SMediaPlayerEditorMediaDetails

private:
	/** Pointer to the MediaPlate that is being viewed. */
	TWeakObjectPtr<UMediaPlateComponent> MediaPlate;
};
