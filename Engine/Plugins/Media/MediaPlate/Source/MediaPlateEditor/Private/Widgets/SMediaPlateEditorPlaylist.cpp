// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMediaPlateEditorPlaylist.h"
#include "MediaPlateComponent.h"
#include "MediaPlaylist.h"

#define LOCTEXT_NAMESPACE "SMediaPlateEditorPlaylist"

void SMediaPlateEditorPlaylist::Construct(const FArguments& InArgs, UMediaPlateComponent& InMediaPlate, const TSharedRef<ISlateStyle>& InStyle)
{
	MediaPlate = &InMediaPlate;

	SMediaPlaylistEditorTracks::Construct(SMediaPlaylistEditorTracks::FArguments(), MediaPlate->GetMediaPlaylist(), InStyle);
}

void SMediaPlateEditorPlaylist::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	// Call parent.
	SMediaPlaylistEditorTracks::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if (MediaPlate != nullptr)
	{
		// Did the playlist change?
		UMediaPlaylist* MediaPlaylist = MediaPlaylistPtr.Get();
		if (MediaPlaylist != MediaPlate->GetMediaPlaylist())
		{
			// Refresh.
			MediaPlaylistPtr = MediaPlate->GetMediaPlaylist();
			RefreshPlaylist();
		}
	}
}

#undef LOCTEXT_NAMESPACE
