// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SMediaPlaylistEditorTracks.h"

class UMediaPlateComponent;

/**
 * Implements the playlist panel of the MediaPlate asset editor.
 */
class SMediaPlateEditorPlaylist
	: public SMediaPlaylistEditorTracks
{
public:

	SLATE_BEGIN_ARGS(SMediaPlateEditorPlaylist) { }
	SLATE_END_ARGS()

public:

	/**
	 * Construct this widget.
	 *
	 * @param InArgs		The declaration data for this widget.
	 * @param InMediaPlate	The MediaPlate to show the details for.
	 * @param InStyleSet	The style set to use.
	 */
	void Construct(const FArguments& InArgs, UMediaPlateComponent& InMediaPlate, const TSharedRef<ISlateStyle>& InStyle);

	//~ SWidget interface
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:

	/** Pointer to the MediaPlate that is being viewed. */
	TWeakObjectPtr<UMediaPlateComponent> MediaPlate;
};
