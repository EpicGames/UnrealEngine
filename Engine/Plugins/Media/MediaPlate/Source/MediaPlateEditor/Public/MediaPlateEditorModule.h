// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "LevelEditor.h"
#include "Logging/LogMacros.h"
#include "Modules/ModuleManager.h"
#include "TickableEditorObject.h"
#include "UObject/ObjectPtr.h"
#include "UObject/WeakObjectPtr.h"

class FDragDropEvent;
class FPlacementModeID;
class UMediaPlateComponent;
class UMediaSource;
struct FAssetData;
struct FPlacementCategoryInfo;

/** Log category for this module. */
DECLARE_LOG_CATEGORY_EXTERN(LogMediaPlateEditor, Log, All);

class FMediaPlateEditorModule : public IModuleInterface, public FTickableEditorObject
{
public:
	
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	/** FTickableEditorObject interface */
	virtual void Tick(float DeltaTime) override;
	virtual TStatId GetStatId() const override { RETURN_QUICK_DECLARE_CYCLE_STAT(FMediaPlateEditorModule, STATGROUP_Tickables); }
	
	/**
	 * Call this when a media plate starts playing so we can track it.
	 */
	void MediaPlateStartedPlayback(TObjectPtr<UMediaPlateComponent> MediaPlate);

	/**
	 * Call this if you want to check if your media source is a drag and drop media source.
	 * If it is, then this will remove it from this module
	 * and you can then change the outer (as it will be transient.)
	 * 
	 * @param MediaSource	Media source to check.
	 * @return				True if it is a drag and drop media source.
	 */
	bool RemoveMediaSourceFromDragDropCache(UMediaSource* MediaSource);

	/**
	 * Delegate called when getting the media plate materials asset paths.
	 * This allows for additional plugins to provide new media plate material templates. 
	 */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnGetMediaPlateMaterialAssetPaths, TArray<FName>& /*OutAssetPaths*/);
	MEDIAPLATEEDITOR_API FOnGetMediaPlateMaterialAssetPaths& OnGetMediaPlateMaterialAssetPaths();
	
private:

	/** Customization names to avoid reusing staticstruct during shutdown. */
	FName MediaPlateName;
	FName MediaPlateResourceName;

	/** Holds all the media plates that are playing. */
	TArray<TWeakObjectPtr<UMediaPlateComponent>> ActiveMediaPlates;

	/** Event Delegate for OnGetMediaPlateMaterialAssetPaths. */
	FOnGetMediaPlateMaterialAssetPaths OnGetMediaPlateMaterialAssetPathsDelegate;

	/** Handle for our track editor. */
	FDelegateHandle TrackEditorBindingHandle;

	/** Maps a drag and drop file to a media source we created from it. */
	TMap<FString, TWeakObjectPtr<UMediaSource>> MapFileToMediaSource;
	/** Caches if real time viewports are enabled. */
	bool bIsRealTimeViewportsEnabled = false;
	/** Stores what we have aded to the placement module. */
	TArray<TOptional<FPlacementModeID>> PlaceActors;

	/** Holds the context menu extender. */
	FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors LevelViewportContextMenuRemoteControlExtender;

	/** Holds the menu extender delegate handle. */
	FDelegateHandle MenuExtenderDelegateHandle;

	/**
	 * Gathers the Info on the Media Plate Place Actors Category.
	 */
	const FPlacementCategoryInfo* GetMediaCategoryRegisteredInfo() const;

	/**
	 * Register items to show up in the Place Actors panel.
	 */
	void RegisterPlacementModeItems();

	/**
	 * Register which categories belong to which section.
	 */
	void RegisterSectionMappings();

	/**
	 * Unregister items in Place Actors panel.
	 */
	void UnregisterPlacementModeItems();

	/**
	 * Called after the engine has initialized.
	 */
	void OnPostEngineInit();

	/**
	 * Extracts an asset for a file if it can.
	 *
	 * @param Files				Files to get the asset from.
	 * @param AssetDataArray	Will add the asset here if it gets one.
	 */
	void ExtractAssetDataFromFiles(const TArray<FString>& Files, TArray<FAssetData>& AssetDataArray);

	/**
	 * Enables real time viewports so media plates can update in the editor.
	 */
	void ForceRealTimeViewports(const bool bEnable);

	/**
	 * Handle adding the menu extender for the actors.
	 */
	TSharedRef<FExtender> ExtendLevelViewportContextMenuForMediaPlate(const TSharedRef<FUICommandList> CommandList, TArray<AActor*> SelectedActors);

	/**
	 * Create all required callbacks for context menu's widget and button creation.
	 */
	void RegisterContextMenuExtender();

	/**
	 * Release context menu related resources such as handles.
	 */
	void UnregisterContextMenuExtender();

	/**
	 * Called when an Actor has been added to a Level
	 */
	void OnLevelActorAdded(AActor* InActor);
};
